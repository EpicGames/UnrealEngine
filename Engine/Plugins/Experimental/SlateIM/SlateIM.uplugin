{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "SlateIM",
	"Description": "An immediate mode wrapper for Slate. Intended for building debugging tools.",
	"Category": "UI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "SlateIM",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SlateIMEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SlateIMInGame",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SlateIMBlueprint",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}