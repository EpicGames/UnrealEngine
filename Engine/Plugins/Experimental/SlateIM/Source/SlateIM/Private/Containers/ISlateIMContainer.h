// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ISlateIMChild.h"
#include "Misc/SlateIMTypeChecking.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

struct FSlateIMSlotData;

class FChildren;
class SWidget;

class ISlateIMContainer : public ISlateIMTypeChecking
{
public:
	virtual FString GetDebugName() { return GetContainer().GetWidget() ? *GetContainer().GetWidget()->GetTypeAsString() : TEXT("UNKNOWN"); }

	virtual int32 GetNumChildren() = 0;
	virtual FSlateIMChild GetChild(int32 Index) = 0;
	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) = 0;

	virtual FSlateIMChild GetContainer() = 0;

	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) = 0;
};
