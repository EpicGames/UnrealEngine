// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImCompoundWidget.h"

#include "Misc/SlateIMSlotData.h"
#include "SImStackBox.h"


SLATE_IMPLEMENT_WIDGET(SImCompoundWidget)

void SImCompoundWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

int32 SImCompoundWidget::GetNumChildren()
{
	if (Container)
	{
		return Container->GetNumChildren();
	}
	
	return GetChildren()->Num();
}

FSlateIMChild SImCompoundWidget::GetChild(int32 Index)
{
	if (Container)
	{
		return Container->GetChild(Index);
	}
	
	if (Index >= 0 && Index < GetChildren()->Num())
	{
		return GetChildren()->GetChildAt(Index);
	}

	return nullptr;
}

void SImCompoundWidget::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	if (Container)
	{
		Container->RemoveUnusedChildren(LastUsedChildIndex);
	}
}

void SImCompoundWidget::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	if (!Container)
	{
		Container = SNew(SImStackBox)
			.Orientation(Orient_Vertical);
		ChildSlot
		.Padding(FMargin(0))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			Container.ToSharedRef()
		];
	}
	
	Container->UpdateChild(Child, Index, AlignmentData);
}

void SImCompoundWidget::SetOrientation(EOrientation InOrientation)
{
	if (Container)
	{
		Container->SetOrientation(InOrientation);
	}
}
