// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMContainer.h"
#include "Widgets/Layout/SBorder.h"

class SImStackBox;

class SImCompoundWidget : public SBorder, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImCompoundWidget, SBorder)
	SLATE_IM_TYPE_DATA(SImCompoundWidget, ISlateIMContainer)
	
public:
	virtual int32 GetNumChildren() override;
	virtual FSlateIMChild GetChild(int32 Index) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;

	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;

	void SetBackgroundImage(const FSlateBrush* BorderBrush)
	{
		SetBorderImage(BorderBrush);
	}

	void SetContentPadding(const FMargin ContentPadding)
	{
		SetPadding(ContentPadding);
	}

	void SetAbsorbMouse(bool bInAbsorb)
	{
		bAbsorbMouse = bInAbsorb;
	}

	void SetOrientation(EOrientation InOrientation);

	virtual FReply OnMouseButtonDown(const FGeometry&, const FPointerEvent&) override
	{
		return bAbsorbMouse ? FReply::Handled() : FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry&, const FPointerEvent&) override
	{
		return bAbsorbMouse ? FReply::Handled() : FReply::Unhandled();
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry&, const FPointerEvent&) override
	{
		return bAbsorbMouse ? FReply::Handled() : FReply::Unhandled();
	}

	virtual FReply OnMouseMove(const FGeometry&, const FPointerEvent&) override
	{
		return bAbsorbMouse ? FReply::Handled() : FReply::Unhandled();
	}

	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return bAbsorbMouse ? FReply::Handled() : FReply::Unhandled();
	}
	
private:
	TSharedPtr<SImStackBox> Container;
	bool bAbsorbMouse = true;
};
