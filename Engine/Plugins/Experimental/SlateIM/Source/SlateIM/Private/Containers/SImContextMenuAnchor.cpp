// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImContextMenuAnchor.h"

#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Misc/SlateIMSlotData.h"


SLATE_IMPLEMENT_WIDGET(SImContextMenuAnchor)

void SImContextMenuAnchor::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SImContextMenuAnchor::Construct(const FArguments& InArgs)
{
}

int32 SImContextMenuAnchor::GetNumChildren()
{
	return GetChildren()->Num();
}

FSlateIMChild SImContextMenuAnchor::GetChild(int32 Index)
{
	if (Index >= 0 && Index < GetNumChildren())
	{
		TSharedRef<SWidget> Child = GetChildren()->GetChildAt(Index);

		if (Child->GetWidgetClass().GetWidgetType() == SBox::StaticWidgetClass().GetWidgetType())
		{
			TSharedRef<SBox> SlotBox = StaticCastSharedRef<SBox>(Child);
			Child = (SlotBox->GetChildren() && SlotBox->GetChildren()->Num() > 0) ? SlotBox->GetChildren()->GetChildAt(0) : SNullWidget::NullWidget;
		}
		
		return Child;
	}

	return nullptr;
}

void SImContextMenuAnchor::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	ChildSlot
	[
		SNullWidget::NullWidget
	];
}

void SImContextMenuAnchor::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	ChildSlot
	.Padding(AlignmentData.Padding)
	.HAlign(AlignmentData.HorizontalAlignment)
	.VAlign(AlignmentData.VerticalAlignment)
	[
		SNew(SBox)
		.MinDesiredWidth(AlignmentData.MinWidth > 0 ? AlignmentData.MinWidth : FOptionalSize())
		.MinDesiredHeight(AlignmentData.MinHeight > 0 ? AlignmentData.MinHeight : FOptionalSize())
		.MaxDesiredWidth(AlignmentData.MaxWidth > 0 ? AlignmentData.MaxWidth : FOptionalSize())
		.MaxDesiredHeight(AlignmentData.MaxHeight > 0 ? AlignmentData.MaxHeight : FOptionalSize())
		[
			Child.GetWidgetRef()
		]
	];
}

void SImContextMenuAnchor::Begin()
{
	bIsDirty = false;
	CurrentMenuIndex = 0;
	CurrentSubMenuLevel = 0;
}

void SImContextMenuAnchor::End()
{
	bIsDirty |= CurrentMenuIndex != MenuHashes.Num();

	if (bIsDirty)
	{
		bIsDirty = false;

		int32 NumMenuItems = CurrentMenuIndex;
		int32 MenuHashCount = MenuHashes.Num();

		if (MenuHashCount > NumMenuItems)
		{
			const int32 NumToRemove = MenuHashCount - NumMenuItems;

			MenuHashes.RemoveAt(CurrentMenuIndex, NumToRemove);
			CheckStates.RemoveAt(CurrentMenuIndex, NumToRemove);
		}

		SubMenuWidgets.Empty();

		int32 StartIndex = 0;
		const int32 MenuLevel = 0;
		MenuWidget = BuildMenu_Recursive(StartIndex, MenuLevel);
	}


	MenuDataList.Empty();
	MenuStringList.Empty();
	ActivatedIndices.Empty();
}

bool SImContextMenuAnchor::AddMenuButton(const FStringView& RowText, const FStringView& ToolTipText)
{
	bool bCurrentStateIgnored = false;
	return AddMenuInternal(RowText, ToolTipText, bCurrentStateIgnored, EMenuType::Button);
}

bool SImContextMenuAnchor::AddMenuCheckButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText)
{
	return AddMenuInternal(RowText, ToolTipText, InOutCurrentState,  EMenuType::Check);
}

bool SImContextMenuAnchor::AddMenuToggleButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText)
{
	return AddMenuInternal(RowText, ToolTipText, InOutCurrentState, EMenuType::Toggle);
}


void SImContextMenuAnchor::AddMenuSeparator()
{
	bool bCurrentStateIgnored = false;
	AddMenuInternal(nullptr, nullptr, bCurrentStateIgnored, EMenuType::Separator);
}

void SImContextMenuAnchor::AddMenuSection(const FStringView& SectionText)
{
	bool bCurrentStateIgnored = false;
	AddMenuInternal(SectionText, nullptr, bCurrentStateIgnored, EMenuType::Section);
}

void SImContextMenuAnchor::BeginSubMenu(const FStringView& SectionText)
{
	bool bCurrentStateIgnored = false;
	AddMenuInternal(SectionText, nullptr, bCurrentStateIgnored, EMenuType::SubMenu);
		
	++CurrentSubMenuLevel;
}

void SImContextMenuAnchor::EndSubMenu()
{
	--CurrentSubMenuLevel;
	checkf(CurrentSubMenuLevel >= 0, TEXT("Too many calls to EndSubMenu. This means you called EndSubMenu more than BeginSubMenu"));
}

void SImContextMenuAnchor::OpenMenu(const FVector2D& InScreenPosition, const FWidgetPath* InWidgetPath)
{
	if (IsMenuOpen())
	{
		return;
	}

	OpenedMenu = FSlateApplication::Get().PushMenu(
		AsShared(), 
		InWidgetPath ? *InWidgetPath : FWidgetPath(), 
		MenuWidget.ToSharedRef(), 
		InScreenPosition, 
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
}

TSharedRef<SWidget> SImContextMenuAnchor::GetMenuContent() const
{
	return MenuWidget.IsValid() ? MenuWidget.ToSharedRef() : SNullWidget::NullWidget;
}

FReply SImContextMenuAnchor::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MenuWidget && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		OpenMenu(MouseEvent.GetScreenSpacePosition(), MouseEvent.GetEventPath());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SImContextMenuAnchor::AddMenuInternal(const FStringView& RowText, const FStringView& ToolTipText, bool& InOutCurrentState, EMenuType MenuType)
{
	bool bIsActivated = false;
	uint64 RowHash = 0;
	if (RowText.Len() > 0)
	{
		RowHash = CityHash64(reinterpret_cast<const char*>(RowText.GetData()), RowText.Len() * sizeof(TCHAR));
	}

	if (ToolTipText.Len() > 0)
	{
		RowHash = CityHash64WithSeed(reinterpret_cast<const char*>(ToolTipText.GetData()), ToolTipText.Len() * sizeof(TCHAR), RowHash);
	}

	FMenuItemData NewMenuItem;

	if (RowText.Len() > 0)
	{
		NewMenuItem.TextOffset = MenuStringList.AddUninitialized(RowText.Len());
		NewMenuItem.TextSize = RowText.Len();
		RowText.CopyString(&MenuStringList[NewMenuItem.TextOffset], RowText.Len(), 0);
	}
	else
	{
		NewMenuItem.TextOffset = INDEX_NONE;
		NewMenuItem.TextSize = 0;
	}

	if (ToolTipText.Len() > 0)
	{
		NewMenuItem.ToolTipOffset = MenuStringList.AddUninitialized(ToolTipText.Len());
		NewMenuItem.ToolTipSize = ToolTipText.Len();
		ToolTipText.CopyString(&MenuStringList[NewMenuItem.ToolTipOffset], ToolTipText.Len(), 0);
	}
	else
	{
		NewMenuItem.ToolTipOffset = INDEX_NONE;
		NewMenuItem.ToolTipSize = 0;
	}


	NewMenuItem.Type = MenuType;
	NewMenuItem.SubMenuLevel = CurrentSubMenuLevel;

	RowHash = CityHash64WithSeed(reinterpret_cast<const char*>(&NewMenuItem), sizeof(NewMenuItem), RowHash);

	MenuDataList.Add(MoveTemp(NewMenuItem));

	bIsDirty |= MenuHashes.Num() > CurrentMenuIndex ? MenuHashes[CurrentMenuIndex] != RowHash : true;
	if (bIsDirty)
	{
		if (MenuHashes.Num() > CurrentMenuIndex)
		{
			MenuHashes[CurrentMenuIndex] = RowHash;
			CheckStates[CurrentMenuIndex] = InOutCurrentState;
		}
		else
		{
			MenuHashes.Add(RowHash);
			CheckStates.Add(InOutCurrentState);
			check(MenuHashes.Num() - 1 == CurrentMenuIndex);
		}
	}
	else
	{
		bIsActivated = ActivatedIndices.Contains(CurrentMenuIndex);

		bool bIsCheckType = MenuType == EMenuType::Check || MenuType == EMenuType::Toggle;
		if (bIsActivated && bIsCheckType)
		{
			InOutCurrentState = CheckStates[CurrentMenuIndex];
		}
		else if (bIsCheckType)
		{
			CheckStates[CurrentMenuIndex] = InOutCurrentState;
		}
	}

	++CurrentMenuIndex;

	return bIsActivated;
}

void SImContextMenuAnchor::OnMenuItemExecuted(int32 MenuIndex, bool bIsCheck)
{
	ActivatedIndices.Add(MenuIndex);

	if (bIsCheck)
	{
		CheckStates[MenuIndex] = !CheckStates[MenuIndex];
	}
}

ECheckBoxState SImContextMenuAnchor::OnGetMenuItemCheckState(int32 MenuIndex)
{
	return CheckStates[MenuIndex] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<SWidget> SImContextMenuAnchor::OnGetSubMenuContent(int32 SubMenuIndex)
{
	//MenuBuilder.AddWidget(SubMenuWidgets[SubMenuIndex], FText::GetEmpty(), false, false);

	return SubMenuWidgets[SubMenuIndex];
}

TSharedRef<SWidget> SImContextMenuAnchor::BuildMenu_Recursive(int32& InOutMenuIndex, int32 MenuLevel)
{
	int32 NumMenuItems = CurrentMenuIndex;

	FMenuBuilder MenuBuilder(true, nullptr, nullptr, false, &FCoreStyle::Get(), false);

	bool bHasCurrentSection = false;

	while(InOutMenuIndex < NumMenuItems)
	{
		const FMenuItemData& MenuItemData = MenuDataList[InOutMenuIndex];

		if (MenuItemData.SubMenuLevel < MenuLevel)
		{
			break;
		}

		FStringView MenuText;
		FStringView ToolTipText;
		if (MenuItemData.TextOffset != INDEX_NONE)
		{
			MenuText = FStringView(&MenuStringList[MenuItemData.TextOffset], MenuItemData.TextSize);
		}

		if (MenuItemData.ToolTipOffset != INDEX_NONE)
		{
			ToolTipText = FStringView(&MenuStringList[MenuItemData.ToolTipOffset], MenuItemData.ToolTipSize);
		}

		switch (MenuItemData.Type)
		{
		case EMenuType::Button:
		case EMenuType::Check:
		case EMenuType::Toggle:
		{
			bool bIsCheckType = MenuItemData.Type == EMenuType::Check || MenuItemData.Type == EMenuType::Toggle;

			FUIAction Action(FExecuteAction::CreateSP(this, &SImContextMenuAnchor::OnMenuItemExecuted, InOutMenuIndex, bIsCheckType));

			EUserInterfaceActionType MenuBuilderType = EUserInterfaceActionType::Button;

			if(bIsCheckType)
			{
				MenuBuilderType = MenuItemData.Type == EMenuType::Check ? EUserInterfaceActionType::Check : EUserInterfaceActionType::ToggleButton;
				Action.GetActionCheckState = FGetActionCheckState::CreateSP(this, &SImContextMenuAnchor::OnGetMenuItemCheckState, InOutMenuIndex);
			}

			MenuBuilder.AddMenuEntry(FText::FromStringView(MenuText), FText::FromStringView(ToolTipText), FSlateIcon(), Action, NAME_None, MenuBuilderType);
		}
		break;

		case EMenuType::Separator:
			MenuBuilder.AddMenuSeparator();
			break;
		case EMenuType::Section:
			if (bHasCurrentSection)
			{
				MenuBuilder.EndSection();
			}

			bHasCurrentSection = true;
			MenuBuilder.BeginSection(NAME_None, FText::FromStringView(MenuText));
			break;

		case EMenuType::SubMenu:
			++InOutMenuIndex;
			TSharedRef<SWidget> SubMenuWidget = BuildMenu_Recursive(InOutMenuIndex, MenuLevel+1);
			int32 SubMenuIndex = SubMenuWidgets.Add(SubMenuWidget);
			MenuBuilder.AddWrapperSubMenu(FText::FromStringView(MenuText), FText::FromStringView(ToolTipText), FOnGetContent::CreateSP(this, &SImContextMenuAnchor::OnGetSubMenuContent, SubMenuIndex), FSlateIcon());
			break;
		}

		// This has already been done in the switch statement for SubMenus.
		if (MenuItemData.Type != EMenuType::SubMenu)
		{
			++InOutMenuIndex;
		}
	}

	if (bHasCurrentSection)
	{
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}
