// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMContainer.h"
#include "Widgets/SCompoundWidget.h"

class FWidgetPath;
class IMenu;

enum class EMenuType : uint8
{
	Button,
	Check,
	Toggle,
	Separator,
	Section,
	SubMenu,
};

class SImContextMenuAnchor : public SCompoundWidget, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImContextMenuAnchor, SCompoundWidget)
	SLATE_IM_TYPE_DATA(SImContextMenuAnchor, ISlateIMContainer)
	
public:
	SLATE_BEGIN_ARGS(SImContextMenuAnchor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual int32 GetNumChildren() override;
	virtual FSlateIMChild GetChild(int32 Index) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;

	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;

	void Begin();
	void End();

	bool AddMenuButton(const FStringView& RowText, const FStringView& ToolTipText);
	bool AddMenuCheckButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText);
	bool AddMenuToggleButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText);
	void AddMenuSeparator();
	void AddMenuSection(const FStringView& SectionText);
	void BeginSubMenu(const FStringView& SectionText);
	void EndSubMenu();
	void OpenMenu(const FVector2D& InScreenPosition, const FWidgetPath* InWidgetPath = nullptr);
	bool IsMenuOpen() const { return OpenedMenu.IsValid();  }
	TSharedRef<SWidget> GetMenuContent() const;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	
private:
	bool AddMenuInternal(const FStringView& RowText, const FStringView& ToolTipText, bool& InOutCurrentState, EMenuType MenuType);
	void OnMenuItemExecuted(int32 MenuIndex, bool bIsCheck);
	ECheckBoxState OnGetMenuItemCheckState(int32 MenuIndex);
	//void OnGetSubMenuContent(FMenuBuilder& MenuBuilder, int32 SubMenuIndex);
	TSharedRef<SWidget> OnGetSubMenuContent(int32 SubMenuIndex);
	TSharedRef<SWidget> BuildMenu_Recursive(int32& InOutMenuIndex, int32 MenuLevel);

	struct FMenuItemData
	{
		int32 TextOffset;
		int32 TextSize;
		int32 ToolTipOffset;
		int32 ToolTipSize;
		int32 SubMenuLevel;
		EMenuType Type;

	};

	TWeakPtr<IMenu> OpenedMenu;
	TSharedPtr<SWidget> MenuWidget;
	TArray<FMenuItemData, TMemStackAllocator<>> MenuDataList;
	TArray<TCHAR, TMemStackAllocator<>> MenuStringList;

	TArray<int32, TInlineAllocator<1>> ActivatedIndices;
	TBitArray<> CheckStates;
	TArray<TSharedRef<SWidget>> SubMenuWidgets;
	TArray<uint64> MenuHashes;
	int32 CurrentMenuIndex = 0;
	int32 CurrentSubMenuLevel = 0;
	bool bIsDirty = false;
};
