// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImPopUp.h"

#include "Misc/SlateIMSlotData.h"
#include "Widgets/Layout/SBox.h"

SLATE_IMPLEMENT_WIDGET(SImPopUp)

void SImPopUp::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SImPopUp::Construct(const FArguments& InArgs)
{
	ChildBox = SNew(SBox)
		[
			InArgs._Content.Widget
		];
	
	SMenuAnchor::Construct(FArguments(InArgs)
		.UseApplicationMenuStack(false)
		.MenuContent(ChildBox.ToSharedRef()));
}

int32 SImPopUp::GetNumChildren()
{
	if (ChildBox.IsValid() && ChildBox->GetChildren() && ChildBox->GetChildren()->Num() > 0)
	{
		TSharedRef<SWidget> ChildWidget = ChildBox->GetChildren()->GetChildAt(0);

		return (ChildWidget == SNullWidget::NullWidget) ? 0 : 1;
	}

	return 0;
}

FSlateIMChild SImPopUp::GetChild(int32 Index)
{
	if (ChildBox.IsValid() && ChildBox->GetChildren())
	{
		return ChildBox->GetChildren()->GetChildAt(Index);
	}

	return nullptr;
}

void SImPopUp::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	TSharedPtr<SWidget> ChildWidget = Child.GetWidget();
	if (ensureMsgf(ChildWidget, TEXT("Invalid child in SlateIM Popup")))
	{
		ChildBox->SetPadding(AlignmentData.Padding);
		ChildBox->SetHAlign(AlignmentData.HorizontalAlignment);
		ChildBox->SetVAlign(AlignmentData.VerticalAlignment);
		ChildBox->SetMinDesiredWidth(AlignmentData.MinWidth > 0 ? AlignmentData.MinWidth : FOptionalSize());
		ChildBox->SetMinDesiredHeight(AlignmentData.MinHeight > 0 ? AlignmentData.MinHeight : FOptionalSize());
		ChildBox->SetMaxDesiredWidth(AlignmentData.MaxWidth > 0 ? AlignmentData.MaxWidth : FOptionalSize());
		ChildBox->SetMaxDesiredHeight(AlignmentData.MaxHeight > 0 ? AlignmentData.MaxHeight : FOptionalSize());
		ChildBox->SetContent(ChildWidget.ToSharedRef());
	}
}

void SImPopUp::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	if (ChildBox.IsValid())
	{
		ChildBox->SetContent(SNullWidget::NullWidget);
	}
}

FVector2D SImPopUp::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D::ZeroVector;
}

