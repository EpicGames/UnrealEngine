// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMContainer.h"
#include "Widgets/Input/SMenuAnchor.h"

#define UE_API SLATEIM_API

class SBox;

class SImPopUp : public SMenuAnchor, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImPopUp, SMenuAnchor)
	SLATE_IM_TYPE_DATA(SImPopUp, ISlateIMContainer)
	
public:
	UE_API void Construct(const FArguments& InArgs);
	
	UE_API virtual int32 GetNumChildren() override;
	UE_API virtual FSlateIMChild GetChild(int32 Index) override;
	UE_API virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	UE_API virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;

	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

private:
	TSharedPtr<SBox> ChildBox;
};

#undef UE_API
