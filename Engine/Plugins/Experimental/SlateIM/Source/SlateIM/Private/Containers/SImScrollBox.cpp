// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImScrollBox.h"
#include "Misc/SlateIMSlotData.h"
#include "Widgets/Layout/SBox.h"

SLATE_IMPLEMENT_WIDGET(SImScrollBox)

void SImScrollBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

int32 SImScrollBox::GetNumChildren()
{
	return ScrollPanel->GetChildren()->Num();
}

FSlateIMChild SImScrollBox::GetChild(int32 Index)
{
	if (Index >= 0 && Index < GetNumChildren())
	{
		TSharedRef<SWidget> Child = ScrollPanel->GetChildren()->GetChildAt(Index);

		if (Child->GetWidgetClass().GetWidgetType() == SBox::StaticWidgetClass().GetWidgetType())
		{
			TSharedRef<SBox> SlotBox = StaticCastSharedRef<SBox>(Child);
			Child = (SlotBox->GetChildren() && SlotBox->GetChildren()->Num() > 0) ? SlotBox->GetChildren()->GetChildAt(0) : SNullWidget::NullWidget;
		}
		
		return Child;
	}

	return nullptr;
}

void SImScrollBox::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	SScrollBox::FSlot* Slot = nullptr;

	if (IsValidSlotIndex(Index))
	{
		Slot = &GetSlot(Index);
	}
	else
	{
		AddSlot().Expose(Slot);
	}

	if (AlignmentData.bAutoSize)
	{
		Slot->SetSizeToAuto();
	}
	else
	{
		Slot->SetSizeToStretchContent(1.0f);
	}

	Slot->SetPadding(AlignmentData.Padding);
	Slot->SetHorizontalAlignment(AlignmentData.HorizontalAlignment);
	Slot->SetVerticalAlignment(AlignmentData.VerticalAlignment);
	if (Slot->GetWidget()->GetWidgetClass().GetWidgetType() != SBox::StaticWidgetClass().GetWidgetType())
	{
		(*Slot)
		[
			SNew(SBox)
			.MinDesiredWidth(AlignmentData.MinWidth > 0 ? AlignmentData.MinWidth : FOptionalSize())
			.MinDesiredHeight(AlignmentData.MinHeight > 0 ? AlignmentData.MinHeight : FOptionalSize())
			.MaxDesiredWidth(AlignmentData.MaxWidth > 0 ? AlignmentData.MaxWidth : FOptionalSize())
			.MaxDesiredHeight(AlignmentData.MaxHeight > 0 ? AlignmentData.MaxHeight : FOptionalSize())
			[
				Child.GetWidgetRef()
			]
		];
	}
	else
	{
		TSharedRef<SBox> SlotBox = StaticCastSharedRef<SBox>(Slot->GetWidget());
		SlotBox->SetMinDesiredWidth(AlignmentData.MinWidth > 0 ? AlignmentData.MinWidth : FOptionalSize());
		SlotBox->SetMinDesiredHeight(AlignmentData.MinHeight > 0 ? AlignmentData.MinHeight : FOptionalSize());
		SlotBox->SetMaxDesiredWidth(AlignmentData.MaxWidth > 0 ? AlignmentData.MaxWidth : FOptionalSize());
		SlotBox->SetMaxDesiredHeight(AlignmentData.MaxHeight > 0 ? AlignmentData.MaxHeight : FOptionalSize());
		SlotBox->SetContent(Child.GetWidgetRef());
	}
}
