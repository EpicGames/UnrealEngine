// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMContainer.h"
#include "Widgets/Layout/SScrollBox.h"

class SImScrollBox : public SScrollBox, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImScrollBox, SScrollBox)
	SLATE_IM_TYPE_DATA(SImScrollBox, ISlateIMContainer)
	
public:
	bool IsValidSlotIndex(int32 Index) const
	{
		return ScrollPanel->Children.IsValidIndex(Index);
	}

	void SetOnUserScrolled(const FOnUserScrolled& InHandler)
	{
		OnUserScrolled = InHandler;
	}

	virtual int32 GetNumChildren() override;
	virtual FSlateIMChild GetChild(int32 Index) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override
	{
		for (int32 IndexToRemove = GetNumChildren() - 1; IndexToRemove > LastUsedChildIndex; --IndexToRemove)
		{
			ScrollPanel->Children.RemoveAt(IndexToRemove);
		}
	}

	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
};
