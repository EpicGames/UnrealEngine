// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImStackBox.h"

#include "Misc/SlateIMSlotData.h"
#include "Widgets/Layout/SBox.h"

SLATE_IMPLEMENT_WIDGET(SImStackBox)

void SImStackBox::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

int32 SImStackBox::GetNumChildren()
{
	return GetChildren()->Num();
}

FSlateIMChild SImStackBox::GetChild(int32 Index)
{
	if (Index >= 0 && Index < GetNumChildren())
	{
		TSharedRef<SWidget> Child = GetChildren()->GetChildAt(Index);

		if (Child->GetWidgetClass().GetWidgetType() == SBox::StaticWidgetClass().GetWidgetType())
		{
			TSharedRef<SBox> SlotBox = StaticCastSharedRef<SBox>(Child);
			Child = (SlotBox->GetChildren() && SlotBox->GetChildren()->Num() > 0) ? SlotBox->GetChildren()->GetChildAt(0) : SNullWidget::NullWidget;
		}
		
		return Child;
	}

	return nullptr;
}

void SImStackBox::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	for (int32 IndexToRemove = GetChildren()->Num() - 1; IndexToRemove > LastUsedChildIndex; --IndexToRemove)
	{
		RemoveSlot(GetChildren()->GetChildAt(IndexToRemove));
	}
}

void SImStackBox::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	SStackBox::FSlot* Slot = nullptr;

	if (IsValidSlotIndex(Index))
	{
		Slot = &GetSlot(Index);
	}
	else
	{
		AddSlot().Expose(Slot);
	}

	if (AlignmentData.bAutoSize)
	{
		Slot->SetSizeToAuto();
	}
	else
	{
		Slot->SetSizeToStretchContent(1.0f);
	}

	Slot->SetHorizontalAlignment(AlignmentData.HorizontalAlignment);
	Slot->SetVerticalAlignment(AlignmentData.VerticalAlignment);
	Slot->SetPadding(AlignmentData.Padding);
	if (Slot->GetWidget()->GetWidgetClass().GetWidgetType() != SBox::StaticWidgetClass().GetWidgetType())
	{
		(*Slot)
		[
			SNew(SBox)
			.MinDesiredWidth(AlignmentData.MinWidth > 0 ? AlignmentData.MinWidth : FOptionalSize())
			.MinDesiredHeight(AlignmentData.MinHeight > 0 ? AlignmentData.MinHeight : FOptionalSize())
			.MaxDesiredWidth(AlignmentData.MaxWidth > 0 ? AlignmentData.MaxWidth : FOptionalSize())
			.MaxDesiredHeight(AlignmentData.MaxHeight > 0 ? AlignmentData.MaxHeight : FOptionalSize())
			[
				Child.GetWidgetRef()
			]
		];
	}
	else
	{
		TSharedRef<SBox> SlotBox = StaticCastSharedRef<SBox>(Slot->GetWidget());
		SlotBox->SetMinDesiredWidth(AlignmentData.MinWidth > 0 ? AlignmentData.MinWidth : FOptionalSize());
		SlotBox->SetMinDesiredHeight(AlignmentData.MinHeight > 0 ? AlignmentData.MinHeight : FOptionalSize());
		SlotBox->SetMaxDesiredWidth(AlignmentData.MaxWidth > 0 ? AlignmentData.MaxWidth : FOptionalSize());
		SlotBox->SetMaxDesiredHeight(AlignmentData.MaxHeight > 0 ? AlignmentData.MaxHeight : FOptionalSize());
		SlotBox->SetContent(Child.GetWidgetRef());
	}
}