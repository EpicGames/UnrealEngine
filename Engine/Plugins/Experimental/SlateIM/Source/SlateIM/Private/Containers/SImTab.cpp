// Copyright Epic Games, Inc. All Rights Reserved.


#include "SImTab.h"

#include "SImStackBox.h"

SLATE_IMPLEMENT_WIDGET(SImTab)

void SImTab::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

int32 SImTab::GetNumChildren()
{
	if (Container)
	{
		return Container->GetNumChildren();
	}
	
	return GetChildren()->Num();
}

FSlateIMChild SImTab::GetChild(int32 Index)
{
	if (Container)
	{
		return Container->GetChild(Index);
	}
	
	if (Index >= 0 && Index < GetChildren()->Num())
	{
		return GetChildren()->GetChildAt(Index);
	}

	return nullptr;
}

void SImTab::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	if (!Container)
	{
		Container = SNew(SImStackBox)
			.Orientation(Orient_Vertical);

		SetContent(Container.ToSharedRef());
	}
	
	Container->UpdateChild(Child, Index, AlignmentData);
}

void SImTab::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	if (Container)
	{
		Container->RemoveUnusedChildren(LastUsedChildIndex);
	}
}

void SImTab::ForceCloseTab()
{
	bCanEverClose = true;
	RequestCloseTab();
}
