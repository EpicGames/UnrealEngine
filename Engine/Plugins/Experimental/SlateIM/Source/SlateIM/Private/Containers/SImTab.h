// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMContainer.h"
#include "Widgets/Docking/SDockTab.h"

#define UE_API SLATEIM_API

class SImStackBox;

class SImTab : public SDockTab, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImTab, SDockTab)
	SLATE_IM_TYPE_DATA(SImTab, ISlateIMContainer)
	
public:
	UE_API virtual int32 GetNumChildren() override;
	UE_API virtual FSlateIMChild GetChild(int32 Index) override;
	UE_API virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
	UE_API virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	UE_API void ForceCloseTab();
	
	bool IsDrawerOpen() const
	{
		return bDrawerOpen;
	}

	void OnDrawerOpened()
	{
		bDrawerOpen = true;
	}

	void OnDrawerClosed()
	{
		bDrawerOpen = false;
	}

	FName TabId = NAME_None;
	FSlateIcon TabIcon;
	FText TabTitle = FText::GetEmpty();

private:
	TSharedPtr<SImStackBox> Container;
	bool bDrawerOpen = false;
};

#undef UE_API
