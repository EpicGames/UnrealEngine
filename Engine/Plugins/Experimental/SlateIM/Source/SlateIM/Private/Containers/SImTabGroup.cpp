// Copyright Epic Games, Inc. All Rights Reserved.


#include "SImTabGroup.h"

#include "Framework/Docking/TabManager.h"
#include "Misc/SlateIMManager.h"
#include "Roots/SlateIMWindowRoot.h"
#include "SImTab.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Text/STextBlock.h"

class FSlateImTabStack : public FTabManager::FStack
{
public:
	bool DoesContainTab(const FName& TabId) const
	{
		return Tabs.ContainsByPredicate([TabId](const FTabManager::FTab& Tab)
		{
			return Tab.TabId.TabType == TabId;
		});
	}

	void SetForegroundTabIfUnset(const FName& TabId)
	{
		if (ForegroundTabId.TabType.IsNone())
		{
			SetForegroundTab(TabId);
		}
	}
	
	bool IsTabForegrounded(const FName& TabId) const
	{
		return ForegroundTabId.TabType == TabId;
	}
};

class FSlateImTabSplitter : public FTabManager::FSplitter
{
public:
	TSharedPtr<FTabManager::FStack> GetChildAsStack(int32 ChildIndex) const
	{
		if (ChildNodes.IsValidIndex(ChildIndex))
		{
			return ChildNodes[ChildIndex]->AsStack();
		}

		return nullptr;
	}

	TSharedPtr<FTabManager::FSplitter> GetChildAsSplitter(int32 ChildIndex) const
	{
		if (ChildNodes.IsValidIndex(ChildIndex))
		{
			return ChildNodes[ChildIndex]->AsSplitter();
		}

		return nullptr;
	}
	
	bool IsTabForegrounded(const FName& TabId) const
	{
		for (const TSharedRef<FTabManager::FLayoutNode>& ChildNode : ChildNodes)
		{
			if (const TSharedPtr<FSlateImTabSplitter>& ChildSplitter = StaticCastSharedPtr<FSlateImTabSplitter>(ChildNode->AsSplitter()))
			{
				if (ChildSplitter->IsTabForegrounded(TabId))
				{
					return true;
				}
			}
			else if (const TSharedPtr<FSlateImTabStack>& ChildStack = StaticCastSharedPtr<FSlateImTabStack>(ChildNode->AsStack()))
			{
				if (ChildStack->IsTabForegrounded(TabId))
				{
					return true;
				}
			}
		}

		return false;
	}

	bool SplitAt(int32 Index, TSharedRef<FLayoutNode> InNode)
	{
		if (!ChildNodes.IsValidIndex(Index) || ChildNodes[Index] != InNode)
		{
			ChildNodes.Insert(InNode, FMath::Min(Index, ChildNodes.Num()));
			return true;
		}

		return false;
	}

	void RemoveChildAt(int32 Index)
	{
		if (ChildNodes.IsValidIndex(Index))
		{
			ChildNodes.RemoveAt(Index);
		}
	}

	bool RemoveUnusedChildren(int32 StartingIndex)
	{
		if (ChildNodes.Num() > StartingIndex)
		{
			ChildNodes.SetNumUninitialized(StartingIndex, EAllowShrinking::No);
			return true;
		}

		return false;
	}
};

class FSlateImTabLayout : public FTabManager::FLayout
{
public:
	bool IsTabForegrounded(const FName& TabId) const
	{
		for (const TSharedRef<FTabManager::FArea>& Area : Areas)
		{
			if (StaticCastSharedPtr<FSlateImTabSplitter>(Area->AsSplitter())->IsTabForegrounded(TabId))
			{
				return true;
			}
		}

		return false;
	}
};

SLATE_IMPLEMENT_WIDGET(SImTabGroup)

void SImTabGroup::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

void SImTabGroup::Construct(const FArguments& InArgs)
{
	TabGroupId = InArgs._TabGroupId;
	
	if (TSharedPtr<SDockTab> ParentTab = SlateIM::FSlateIMManager::Get().FindMostRecentContainer<SImTab>())
	{
		TabManager = FGlobalTabmanager::Get()->NewTabManager(ParentTab.ToSharedRef());
	}
	else if (SlateIM::FSlateIMManager::Get().HasCurrentRoot()
		&& SlateIM::FSlateIMManager::Get().GetCurrentRoot().RootWidget.IsValid())
	{
		TabManager = FGlobalTabmanager::Get()->NewTabManager(SlateIM::FSlateIMManager::Get().GetCurrentRoot().RootWidget->GetRootTab());
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Missing root widget when creating a tab. Are you missing a Begin Root?"));
		return;
	}

	const TSharedRef<FTabManager::FArea> PrimaryArea = TabManager->NewPrimaryArea();
	WorkingLayoutNodeStack.Emplace(FImTabLayoutNode{ PrimaryArea });
	TabLayout = FTabManager::NewLayout(TabGroupId)->AddArea(PrimaryArea);
}

SImTabGroup::~SImTabGroup()
{
	// Force-close the tabs to handle any that may have been pulled into their own windows
	for (const FImTab& Tab : Tabs)
	{
		if (Tab.TabWidget.IsValid())
		{
			Tab.TabWidget->ForceCloseTab();
		}
	}
}

void SImTabGroup::BeginTabGroupUpdates()
{
	checkf(WorkingLayoutNodeStack.Num() == 1, TEXT("The Tab Group's WorkingLayoutNodeStack should have exactly 1 entry. Are your Begin and End function calls mismatched?"));
	WorkingLayoutNodeStack.SetNumUninitialized(1, EAllowShrinking::No);
	WorkingLayoutNodeStack[0].CurrentChildIndex = 0;
}

void SImTabGroup::FinalizeTabGroup()
{
	if (!bIsLayoutUpToDate)
	{
		check(TabLayout.IsValid());
		bIsLayoutUpToDate = true;
		
		TSharedPtr<SWindow> TabWindow;
		const TSharedPtr<ISlateIMRoot>& RootWidget = SlateIM::FSlateIMManager::Get().GetCurrentRoot().RootWidget;
		if (RootWidget.IsValid() && RootWidget->IsA<FSlateIMWindowRoot>())
		{
			TabWindow = StaticCastSharedPtr<FSlateIMWindowRoot>(RootWidget)->GetWindow();
		}

		// TODO - Save layout to restore from (part of overall SlateIM save/restore user's UI state)
		ChildSlot
		[
			TabManager->RestoreFrom(TabLayout.ToSharedRef(), TabWindow).ToSharedRef()
		];
	}
}

int32 SImTabGroup::GetNumChildren()
{
	return Tabs.Num();
}

FSlateIMChild SImTabGroup::GetChild(int32 Index)
{
	return (Tabs.IsValidIndex(Index) && Tabs[Index].TabWidget.IsValid()) ? FSlateIMChild(Tabs[Index].TabWidget.ToSharedRef()) : FSlateIMChild();
}

void SImTabGroup::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	TSharedPtr<SImTab> TabWidget = Child.GetWidget<SImTab>();
	if (!ensureMsgf(TabWidget.IsValid(), TEXT("Attempting to add a non-tab widget to a Tab Group, only Tabs can be added to Tab Groups")))
	{
		return;
	}
	
	if (!ensureMsgf(!WorkingLayoutNodeStack.IsEmpty(), TEXT("The Tab Group's WorkingLayoutNodeStack should not be empty. Are your Begin and End function calls mismatched?")))
	{
		return;
	}
	
	const TSharedPtr<FSlateImTabStack> Stack = StaticCastSharedPtr<FSlateImTabStack>(WorkingLayoutNodeStack.Last().Node->AsStack());
	if (!ensureMsgf(Stack.IsValid(), TEXT("Cannot add a tab without a Tab Stack. Are your Begin and End function calls mismatched?")))
	{
		return;
	}

	if (!Stack->DoesContainTab(TabWidget->TabId))
	{
		Stack->AddTab(TabWidget->TabId, ETabState::OpenedTab);
		Stack->SetForegroundTabIfUnset(TabWidget->TabId);
		bIsLayoutUpToDate = false;
	}

	if (!Tabs.IsValidIndex(Index))
	{
		Tabs.SetNum(Index + 1);
		bIsLayoutUpToDate = false;
	}

	if (Tabs[Index].TabId != TabWidget->TabId)
	{
		Tabs[Index].TabId = TabWidget->TabId;
		bIsLayoutUpToDate = false;
	}
			
	if (Tabs[Index].TabWidget != TabWidget)
	{
		Tabs[Index].TabWidget = TabWidget;
		bIsLayoutUpToDate = false;
	}

	check(TabManager.IsValid());
	if (!TabManager->HasTabSpawner(TabWidget->TabId))
	{
		FTabSpawnerEntry& Spawner = TabManager->RegisterTabSpawner(TabWidget->TabId, FOnSpawnTab::CreateSP(this, &SImTabGroup::SpawnTab));
		Spawner.SetReuseTabMethod(FOnFindTabToReuse::CreateSP(this, &SImTabGroup::ReuseTab));
		Spawner.SetDisplayName(TabWidget->TabTitle);
		Spawner.SetIcon(TabWidget->TabIcon);
		bIsLayoutUpToDate = false;
	}
}

void SImTabGroup::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	check(TabManager.IsValid());
	for (int32 ChildIndex = LastUsedChildIndex + 1; ChildIndex < Tabs.Num(); ++ChildIndex)
	{
		TabManager->UnregisterTabSpawner(Tabs[ChildIndex].TabId);
		bIsLayoutUpToDate = false;
	}

	Tabs.SetNum(LastUsedChildIndex + 1);
}

FName SImTabGroup::GetTabGroupId() const
{
	return TabLayout.IsValid() ? TabLayout->GetLayoutName() : NAME_None;
}

bool SImTabGroup::IsTabForegrounded(const FName& TabId) const
{
	// If the Tab Manager is going to be restoring from layout this frame, query the layout directly
	if (!bIsLayoutUpToDate)
	{
		check(TabLayout.IsValid());
		return StaticCastSharedPtr<FSlateImTabLayout>(TabLayout)->IsTabForegrounded(TabId);
	}
	else
	{
		for (const FImTab& Tab : Tabs)
		{
			if (Tab.TabWidget.IsValid() && Tab.TabId == TabId)
			{
				if (Tab.TabWidget->IsForeground())
				{
					return true;
				}
				
				return (Tab.TabWidget->IsInSidebar() && Tab.TabWidget->IsDrawerOpen());
			}
		}
	}

	return false;
}

void SImTabGroup::SetSizeCoefficient(float SizeCoefficient)
{
	CurrentSizeCoefficient = SizeCoefficient;
}

void SImTabGroup::BeginTabSplitter(EOrientation Orientation)
{
	TSharedRef<FTabManager::FSplitter> TabSplitter = [this, Orientation]()
	{
		checkf(!WorkingLayoutNodeStack.IsEmpty(), TEXT("The Tab Group's WorkingLayoutNodeStack should not be empty. Are your Begin and End function calls mismatched?"));
		FImTabLayoutNode& ParentNode = WorkingLayoutNodeStack.Last();
		TSharedPtr<FSlateImTabSplitter> ParentSplitter = StaticCastSharedPtr<FSlateImTabSplitter>(ParentNode.Node->AsSplitter());
		checkf(ParentSplitter.IsValid(), TEXT("Tab Splitters can only be added to the Tab Group or other Tab Splitters. Are your Begin and End function calls mismatched?"));
		
		if (TSharedPtr<FTabManager::FSplitter> ExistingSplitter = ParentSplitter->GetChildAsSplitter(ParentNode.CurrentChildIndex))
		{
			if (ExistingSplitter->GetOrientation() != Orientation)
			{
				bIsLayoutUpToDate = false;
				ExistingSplitter->SetOrientation(Orientation);
			}
			return ExistingSplitter.ToSharedRef();
		}

		ParentSplitter->RemoveChildAt(ParentNode.CurrentChildIndex);
		bIsLayoutUpToDate = false;
		TSharedRef<FTabManager::FSplitter> TabSplitter = FTabManager::NewSplitter()
			->SetSizeCoefficient(CurrentSizeCoefficient)
			->SetOrientation(Orientation);
		return TabSplitter;
	}();

	// Reset the size coefficient after using it
	CurrentSizeCoefficient = 1.f;
	WorkingLayoutNodeStack.Emplace(FImTabLayoutNode{ TabSplitter });
}

void SImTabGroup::BeginTabStack()
{
	TSharedRef<FTabManager::FStack> TabStack = [this]()
	{
		checkf(!WorkingLayoutNodeStack.IsEmpty(), TEXT("The Tab Group's WorkingLayoutNodeStack should not be empty. Are your Begin and End function calls mismatched?"));
		FImTabLayoutNode& ParentNode = WorkingLayoutNodeStack.Last();
		TSharedPtr<FSlateImTabSplitter> ParentSplitter = StaticCastSharedPtr<FSlateImTabSplitter>(ParentNode.Node->AsSplitter());
		checkf(ParentSplitter.IsValid(), TEXT("Tab Stacks can only be added to the Tab Group or other Tab Splitters. Are your Begin and End function calls mismatched?"));
		
		if (TSharedPtr<FTabManager::FStack> ExistingStack = ParentSplitter->GetChildAsStack(ParentNode.CurrentChildIndex))
		{
			return ExistingStack.ToSharedRef();
		}

		ParentSplitter->RemoveChildAt(ParentNode.CurrentChildIndex);
		bIsLayoutUpToDate = false;
		TSharedRef<FTabManager::FStack> TabStack = FTabManager::NewStack()
			->SetSizeCoefficient(CurrentSizeCoefficient)
			->SetHideTabWell(false);
		return TabStack;
	}();
	
	// Reset the size coefficient after using it
	CurrentSizeCoefficient = 1.f;
	WorkingLayoutNodeStack.Emplace(FImTabLayoutNode{ TabStack });
}

void SImTabGroup::EndTabLayoutNode()
{
	if (ensureMsgf(!WorkingLayoutNodeStack.IsEmpty(), TEXT("The Tab Group's WorkingLayoutNodeStack should not be empty. Are your Begin and End function calls mismatched?")))
	{
		const FImTabLayoutNode& Node = WorkingLayoutNodeStack.Pop(EAllowShrinking::No);
		if (ensureMsgf(!WorkingLayoutNodeStack.IsEmpty(), TEXT("The Tab Group's LayoutNodeStack should not be empty. Are your Begin and End function calls mismatched?")))
		{
			// Remove unused children in the Node
			if (const TSharedPtr<FSlateImTabSplitter> NodeSplitter = StaticCastSharedPtr<FSlateImTabSplitter>(Node.Node->AsSplitter()))
			{
				if (NodeSplitter->RemoveUnusedChildren(Node.CurrentChildIndex))
				{
					bIsLayoutUpToDate = false;
				}
			}
			
			FImTabLayoutNode& ParentNode = WorkingLayoutNodeStack.Last();
			if (const TSharedPtr<FSlateImTabSplitter> Splitter = StaticCastSharedPtr<FSlateImTabSplitter>(ParentNode.Node->AsSplitter()))
			{
				if (Splitter->SplitAt(ParentNode.CurrentChildIndex, Node.Node))
				{
					bIsLayoutUpToDate = false;
				}
				++ParentNode.CurrentChildIndex;
			}
			else if (ParentNode.Node->AsStack())
			{
				ensureMsgf(false, TEXT("Tab Stacks can only contain tabs. Are your Begin and End function calls mismatched?"));
			}
			else
			{
				ensureMsgf(false, TEXT("Unhandled tab Layout Node type. Are your Begin and End function calls mismatched?"));
			}
		}
	}
}

TSharedRef<SDockTab> SImTabGroup::SpawnTab(const FSpawnTabArgs& Args)
{
	if (TSharedPtr<SDockTab> ExistingTab = ReuseTab(Args.GetTabId()))
	{
		return ExistingTab.ToSharedRef();
	}
	
	return SNew(SDockTab)
	[
		SNew(STextBlock)
		.Text(FText::Format(NSLOCTEXT("SlateIM", "InvalidTabIdFormat", "Attempting to display invalid Tab Id: {0}"), FText::FromString(Args.GetTabId().ToString())))
	];
}

TSharedPtr<SDockTab> SImTabGroup::ReuseTab(const FTabId& TabId) const
{
	const FImTab* TabPtr = Tabs.FindByPredicate([TabId](const FImTab& Tab)
	{
		return Tab.TabId == TabId.TabType;
	});

	if (TabPtr != nullptr && TabPtr->TabWidget.IsValid())
	{
		return TabPtr->TabWidget.ToSharedRef();
	}

	return nullptr;
}
