// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "ISlateIMContainer.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SLATEIM_API

class SImTab;

class SImTabGroup : public SCompoundWidget, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImTabGroup, SCompoundWidget)
	SLATE_IM_TYPE_DATA(SImTabGroup, ISlateIMContainer)
	
public:
	SLATE_BEGIN_ARGS(SImTabGroup)
		{}
		SLATE_ARGUMENT(FName, TabGroupId)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual ~SImTabGroup() override;

	UE_API void BeginTabGroupUpdates();
	UE_API void FinalizeTabGroup();

	UE_API virtual int32 GetNumChildren() override;
	UE_API virtual FSlateIMChild GetChild(int32 Index) override;
	UE_API virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
	UE_API virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	UE_API FName GetTabGroupId() const;
	UE_API bool IsTabForegrounded(const FName& TabId) const;

	UE_API void SetSizeCoefficient(float SizeCoefficient);
	
	UE_API void BeginTabSplitter(EOrientation Orientation);
	UE_API void BeginTabStack();
	UE_API void EndTabLayoutNode();

	UE_API TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);
	UE_API TSharedPtr<SDockTab> ReuseTab(const FTabId& TabId) const;

private:
	struct FImTab
	{
		FName TabId = NAME_None;
		TSharedPtr<SImTab> TabWidget;
	};
	struct FImTabLayoutNode
	{
		TSharedRef<FTabManager::FLayoutNode> Node;
		int32 CurrentChildIndex = 0;
	};
	TSharedPtr<FTabManager> TabManager;
	TSharedPtr<FTabManager::FLayout> TabLayout;
	TArray<FImTabLayoutNode> WorkingLayoutNodeStack;
	TArray<FImTab> Tabs;
	FName TabGroupId = NAME_None;
	float CurrentSizeCoefficient = 1.f;
	bool bIsLayoutUpToDate = false;
};

#undef UE_API
