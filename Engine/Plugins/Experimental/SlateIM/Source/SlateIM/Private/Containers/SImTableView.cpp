// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImTableView.h"

#include "Misc/SlateIMLogging.h"
#include "Misc/SlateIMSlotData.h"

SLATE_IMPLEMENT_WIDGET(SImTableHeader)
SLATE_IMPLEMENT_WIDGET(SImTableRow)
SLATE_IMPLEMENT_WIDGET(SImTableView)

namespace SlateIM::Private
{
	const FName ExpanderColumnName = TEXT("Expander");
	const uint32 NoRowId = 0;
}

void SImTableHeader::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

void FSlateIMTableHeader::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	const int32 NumChildren = FMath::Min(Children.Num(), LastUsedChildIndex + 1);

	if (Children.Num() != NumChildren)
	{
		Children.SetNum(NumChildren);

		if (OwningTable)
		{
			if (TSharedPtr<SHeaderRow> HeaderRow = OwningTable->GetHeaderRow())
			{
				const TIndirectArray<SHeaderRow::FColumn>& Columns = HeaderRow->GetColumns();

				TArray<FName> ColumnsToRemove;
				ColumnsToRemove.Reserve(FMath::Max(0, Columns.Num() - LastUsedChildIndex - 1));

				for (int32 Index = Columns.Num() - 1; Index > LastUsedChildIndex; --Index)
				{
					ColumnsToRemove.Add(Columns[Index].ColumnId);
				}

				for (const FName& ColumnId : ColumnsToRemove)
				{
					HeaderRow->RemoveColumn(ColumnId);
				}
			}
		}
	}
}

void FSlateIMTableHeader::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	if (Children.IsValidIndex(Index))
	{
		Children[Index] = Child;
	}
	else
	{
		Children.Add(Child);
	}
}

TSharedPtr<SWidget> FSlateIMTableHeader::GetAsWidget()
{
	if (OwningTable)
	{
		return OwningTable->GetHeaderRow();
	}

	return nullptr;
}

TSharedPtr<SImTableView> FSlateIMTableHeader::GetOwningTable() const
{
	return OwningTable;
}

bool FSlateIMRowExpansionStateData::HasSavedExpansionState(uint32 InRowId) const
{
	if (InRowId != SlateIM::Private::NoRowId)
	{
		return ExpansionStates.Contains(InRowId);
	}

	return false;
}

bool FSlateIMRowExpansionStateData::GetSavedExpansionState(uint32 InRowId, bool bDefaultState) const
{
	if (InRowId != SlateIM::Private::NoRowId)
	{
		if (const bool* ExpandedState = ExpansionStates.Find(InRowId))
		{
			return *ExpandedState;
		}
	}

	return bDefaultState;
}

void FSlateIMRowExpansionStateData::SetSavedExpansionState(uint32 InRowId, bool bState)
{
	if (InRowId != SlateIM::Private::NoRowId)
	{
		ExpansionStates.FindOrAdd(InRowId) = bState;
	}
}

void FSlateIMTableHeader::SetOwningTable(const TSharedPtr<SImTableView>& InOwningTable)
{
	OwningTable = InOwningTable;
}

void FSlateIMTableRow::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	const int32 NumChildren = FMath::Min(Children.Num(), LastUsedChildIndex + 1);
	if (Children.Num() != NumChildren)
	{
		Children.SetNum(NumChildren);

		// Modifying the child count requires a rebuild to handle the case where we need to create/destroy the expander arrow and child widgets
		if (OwningTable)
		{
			OwningTable->RequestRebuild();
		}
	}
}

void FSlateIMTableRow::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	if (TSharedPtr<FSlateIMTableRow> ChildRow = Child.GetChild<FSlateIMTableRow>())
	{
		ChildRow->UpdateColumnCount(ColumnCount);
		ChildRow->SetOwningTable(OwningTable);
	}

	if (Children.IsValidIndex(Index))
	{
		Children[Index] = Child;
	}
	else
	{
		Children.Add(Child);

		// Modifying the child count requires a rebuild to handle the case where we need to create/destroy the expander arrow and child widgets
		if (OwningTable)
		{
			OwningTable->RequestRebuild();
		}
	}
}

TSharedPtr<SWidget> FSlateIMTableRow::GetAsWidget()
{
	if (OwningTable)
	{
		return StaticCastSharedPtr<SImTableRow>(OwningTable->WidgetFromItem(AsShared()));
	}

	return nullptr;
}

void FSlateIMTableRow::GetChildRows(TArray<TSharedRef<FSlateIMTableRow>>& OutRows) const
{
	for (const FSlateIMChild& Child : Children)
	{
		if (TSharedPtr<FSlateIMTableRow> ChildRow = Child.GetChild<FSlateIMTableRow>())
		{
			OutRows.Add(ChildRow.ToSharedRef());
		}
	}
}

TSharedPtr<ISlateIMContainer> FSlateIMTableRow::GetParent() const
{
	return Parent.Pin();
}

void FSlateIMTableRow::SetParent(TSharedPtr<ISlateIMContainer> InParent)
{
	Parent = InParent;
}

int32 FSlateIMTableRow::CountCellWidgetsUpToIndex(int32 Index) const
{
	int32 CellCount = 0;
	
	for (int32 i = 0; i < Children.Num() && i <= Index; ++i)
	{
		const FSlateIMChild& Child = Children[i];
		if (TSharedPtr<SWidget> ChildRow = Child.GetWidget())
		{
			++CellCount;
		}
	}

	return CellCount;
}

int32 FSlateIMTableRow::FindColumnIndex(const FName& ColumnName) const
{
	return OwningTable ? OwningTable->FindColumnIndex(ColumnName) : INDEX_NONE;
}

TSharedRef<SWidget> FSlateIMTableRow::GetCellWidget(int32 CellIndex) const
{
	int32 CellCount = 0;
	for (const FSlateIMChild& Child : Children)
	{
		if (TSharedPtr<SWidget> ChildRow = Child.GetWidget())
		{
			if (CellIndex == CellCount)
			{
				return ChildRow.ToSharedRef();
			}

			++CellCount;
		}
	}

	return SNullWidget::NullWidget;
}

void FSlateIMTableRow::UpdateColumnCount(int32 NewColumnCount)
{
	ColumnCount = NewColumnCount;

	for (const FSlateIMChild& Child : Children)
	{
		if (TSharedPtr<FSlateIMTableRow> ChildRow = Child.GetChild<FSlateIMTableRow>())
		{
			ChildRow->UpdateColumnCount(ColumnCount);
		}
	}
}

TSharedPtr<SImTableView> FSlateIMTableRow::GetOwningTable() const
{
	return OwningTable;
}

void FSlateIMTableRow::SetOwningTable(const TSharedPtr<SImTableView>& InOwningTable)
{
	OwningTable = InOwningTable;

	for (const FSlateIMChild& Child : Children)
	{
		if (TSharedPtr<FSlateIMTableRow> ChildRow = Child.GetChild<FSlateIMTableRow>())
		{
			ChildRow->SetOwningTable(OwningTable);
		}
	}
}

bool FSlateIMTableRow::IsExpanded()
{
	if (OwningTable)
	{
		return OwningTable->IsItemExpanded(AsShared());
	}

	return false;
}

bool FSlateIMTableRow::HasChildRows() const
{
	for (const FSlateIMChild& Child : Children)
	{
		if (TSharedPtr<FSlateIMTableRow> ChildRow = Child.GetChild<FSlateIMTableRow>())
		{
			return true;
		}
	}

	return false;
}

bool FSlateIMTableRow::ShouldDisplayExpander() const
{
	return OwningTable && OwningTable->IsTree();
}

bool FSlateIMTableRow::AreTableRowContentsRequired()
{
	if (OwningTable)
	{
		if (OwningTable->WidgetFromItem(AsShared()).IsValid())
		{
			// This row has a live widget so the contents are required
			return true;
		}

		const float NumLiveWidgets = OwningTable->GetNumLiveWidgets();
		const float CurrentScrollOffset = OwningTable->GetScrollOffset();

		// Require content from rows within one "page" of the current scroll position in either direction
		// This is probably a bigger window than necessary but should handle large scroll movements and should be small enough to not impact perf
		const float MinNearlyLiveWidget = FMath::Max(0.f, CurrentScrollOffset - NumLiveWidgets);
		const float MaxNearlyLiveWidget = CurrentScrollOffset + (2.f * NumLiveWidgets);
		const float ThisRowIndex = OwningTable->GetRowCount() - 1;
		if (ThisRowIndex >= MinNearlyLiveWidget && ThisRowIndex <= MaxNearlyLiveWidget)
		{
			return true;
		}
	}
	
	return false;
}

void SImTableRow::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

void SImTableRow::Construct(const FSuperRowType::FArguments& InArgs, const TSharedRef<SImTableView>& InOwnerTableView, const TSharedRef<FSlateIMTableRow>& InTableRow)
{
	TableRow = InTableRow;
	SMultiColumnTableRow<TSharedRef<FSlateIMTableRow>>::Construct(InArgs, InOwnerTableView);
}

TSharedRef<SWidget> SImTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SlateIM::Private::ExpanderColumnName)
	{
		return SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this))
			.StyleSet(ExpanderStyleSet)
			.ShouldDrawWires(true);
	}
	else if (TableRow)
	{
		const int32 ColumnIndex = TableRow->FindColumnIndex(ColumnName);
		if (ColumnIndex == 0 && TableRow->ShouldDisplayExpander())
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this))
					.StyleSet(ExpanderStyleSet)
					.ShouldDrawWires(true)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					TableRow->GetCellWidget(ColumnIndex)
				];
		}
		else
		{
			return TableRow->GetCellWidget(ColumnIndex);
		}
	}

	return SNullWidget::NullWidget;
}

void FSlateIMTableBody::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	const int32 NumChildren = FMath::Min(Children.Num(), LastUsedChildIndex + 1);
	if (Children.Num() != NumChildren)
	{
		Children.SetNum(NumChildren);

		// Modifying the child count requires a rebuild to handle the new column title widgets
		if (OwningTable)
		{
			OwningTable->RequestRebuild();
		}
	}
}

void FSlateIMTableBody::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	TSharedPtr<FSlateIMTableRow> ChildRow = Child.GetChild<FSlateIMTableRow>();

	if (!ensureMsgf(ChildRow, TEXT("Table bodies may only contain rows.")))
	{
		return;
	}

	if (OwningTable)
	{
		ChildRow->UpdateColumnCount(OwningTable->GetColumnCount());
	}

	ChildRow->SetOwningTable(OwningTable);

	if (Children.IsValidIndex(Index))
	{
		Children[Index] = ChildRow.ToSharedRef();
	}
	else
	{
		Children.Add(ChildRow.ToSharedRef());

		// Modifying the child count requires a rebuild to handle the case where we need to create/destroy the expander arrow and child widgets
		if (OwningTable)
		{
			OwningTable->RequestRebuild();
		}
	}
}

TSharedPtr<SImTableView> FSlateIMTableBody::GetOwningTable() const
{
	return OwningTable;
}

void FSlateIMTableBody::SetOwningTable(const TSharedPtr<SImTableView>& InOwningTable)
{
	OwningTable = InOwningTable;
}

TSharedPtr<FSlateIMTableRow> FSlateIMTableBody::GetRow(int32 Index)
{
	if (Children.IsValidIndex(Index))
	{
		return Children[Index];
	}
	
	return nullptr;
}

bool FSlateIMTableBody::IsTree() const
{
	for (const TSharedRef<FSlateIMTableRow>& TableRow : Children)
	{
		if (TableRow->HasChildRows())
		{
			return true;
		}
	}

	return false;
}

const TArray<TSharedRef<FSlateIMTableRow>>& FSlateIMTableBody::GetTableRows() const
{
	return Children;
}

void SImTableView::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

void SImTableView::Construct(const FArguments& InArgs)
{
	Header = MakeShared<FSlateIMTableHeader>();
	Header->SetOwningTable(SharedThis(this));

	Body = MakeShared<FSlateIMTableBody>();
	Body->SetOwningTable(SharedThis(this));

	Super::Construct(
		FArguments(InArgs)
		.TreeItemsSource(&Body->GetTableRows())  // Cannot set IItemsSource here, but still needs data.
		.HeaderRow(SNew(SImTableHeader, Header.ToSharedRef()))
		.OnGenerateRow(this, &SImTableView::GenerateRow)
		.OnGetChildren(this, &SImTableView::GatherChildren)
		.OnExpansionChanged(this, &SImTableView::HandleOnExpansionChanged)
	);

	SetRootItemsSource(MakeUnique<FRowSource>(*this));
	RequestListRefresh();
}

FSlateIMChild SImTableView::GetChild(int32 Index)
{
	switch (Index)
	{
		case 0:
			return FSlateIMChild(Header.ToSharedRef());

		case 1:
			return FSlateIMChild(Body.ToSharedRef());

		default:
			return {};
	}
}

void SImTableView::RemoveUnusedChildren(int32 LastUsedChildIndex)
{
	// Cannot reset the children.
}

void SImTableView::UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData)
{
	switch (Index)
	{
		case 0:
			if (TSharedPtr<FSlateIMTableHeader> ChildHeader = Child.GetChild<FSlateIMTableHeader>())
			{
				Header = ChildHeader;

				if (HeaderRow)
				{
					HeaderRow->RefreshColumns();
				}
			}
			break;

		case 1:
			if (TSharedPtr<FSlateIMTableBody> ChildBody = Child.GetChild<FSlateIMTableBody>())
			{
				Body = ChildBody;
				RequestLayoutRefresh();
			}
			break;

		default:
			checkNoEntry();
			break;
	}
}

float SImTableView::GetNumLiveWidgets() const
{
	if (ItemsPanel)
	{
		// NumLiveWidgets can be 0 during updates, so we use the last valid value we had
		const int32 NumLiveWidgets = STreeView<TSharedRef<FSlateIMTableRow>>::GetNumLiveWidgets();

		if (NumLiveWidgets > 0)
		{
			CachedNumLiveWidgets = NumLiveWidgets;
		}
	}

	return CachedNumLiveWidgets;
}

TSharedPtr<FSlateIMTableHeader> SImTableView::GetHeader() const
{
	return Header;
}

TSharedPtr<FSlateIMTableBody> SImTableView::GetBody() const
{
	return Body;
}

void SImTableView::AddColumn(const FName& ColumnID, const FStringView& ColumnToolTip, const FSlateIMSlotData& SlotData)
{
	// Only add columns that don't already exist
	if (HeaderRow)
	{
		// TODO - Handle changes to SlotData after construction
		int32 ColumnIdx = HeaderRow->FindColumnIndex(ColumnID);
		TSharedRef<SWidget> Content = SNullWidget::NullWidget;

		if (ColumnIdx == INDEX_NONE)
		{
			AddColumn_Internal(ColumnID, ColumnToolTip, SlotData);
		}
		else if (ColumnIdx != ColumnCount)
		{
			HeaderRow->SwapColumns(ColumnIdx, ColumnCount);
			HeaderRow->RefreshColumns();
		}		
	}

	++ColumnCount;
}

void SImTableView::EndColumnUpdates()
{
	if (Header && HeaderRow)
	{
		TIndirectArray<SHeaderRow::FColumn>& Columns = const_cast<TIndirectArray<SHeaderRow::FColumn>&>(HeaderRow->GetColumns());

		for (int32 Index = 0; Index < Columns.Num(); ++Index)
		{
			Columns[Index].HeaderContent.Widget = Header->GetChild(Index).GetWidgetRef();
		}
	}
}

void SImTableView::BeginTableUpdates()
{
	ColumnCount = 0;
	RowCount = 0;
}

void SImTableView::EndTableUpdates()
{
	if (DirtyState == NeedsRefresh)
	{
		DirtyState = Clean;
		RequestListRefresh();

		if (HeaderRow)
		{
			HeaderRow->RefreshColumns();
		}
	}
	else if (DirtyState == NeedsRebuild)
	{
		DirtyState = Clean;
		RebuildList();

		if (HeaderRow)
		{
			HeaderRow->RefreshColumns();
		}
	}
}

void SImTableView::BeginTableContent()
{
	if (ColumnCount == 0)
	{
		// TODO - Do we force create a column here or can we hide the header and automatically treat everything like a list?
	}
	UpdateColumns();
}

void SImTableView::UpdateColumns()
{
	if (HeaderRow)
	{
		const int32 MaxColumnIndex = HeaderRow->GetColumns().Num();
		if (ColumnCount < MaxColumnIndex)
		{
			HeaderRow->RemoveColumns(ColumnCount, MaxColumnIndex - ColumnCount);
		}
	}
}

void SImTableView::SetTableRowStyle(const FTableRowStyle* InRowStyle)
{
	RowStyle = InRowStyle;
}

bool SImTableView::IsTree() const
{
	return Body->IsTree();
}

void SImTableView::RequestRefresh()
{
	DirtyState = NeedsRefresh;
}

void SImTableView::RequestRebuild()
{
	DirtyState = NeedsRebuild;
}

TSharedRef<ITableRow> SImTableView::GenerateRow(TSharedRef<FSlateIMTableRow> InTableRow, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SImTableRow, SharedThis(this), InTableRow)
		.Style(RowStyle ? RowStyle : &FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"));
}

void SImTableView::GatherChildren(TSharedRef<FSlateIMTableRow> Row, TArray<TSharedRef<FSlateIMTableRow>>& OutChildren) const
{
	Row->GetChildRows(OutChildren);
}

TSharedPtr<FSlateIMTableRow> SImTableView::GetRow(int32 Index)
{ 
	return Body->GetRow(Index);
}

int32 SImTableView::GetRowLinearizedIndex(const TSharedRef<FSlateIMTableRow>& Row) const
{
	return LinearizedItems.IndexOfByKey(Row);
}

bool SImTableView::IsSameColumnCount(const TSharedRef<FSlateIMTableRow>& Row) const
{
	return Row->GetColumnCount() == ColumnCount;
}

int32 SImTableView::FindColumnIndex(const FName& ColumnName) const
{
	return HeaderRow ? HeaderRow->FindColumnIndex(ColumnName) : INDEX_NONE;
}

void SImTableView::OnRowAdded()
{
	++RowCount;
}

int32 SImTableView::GetRowCount() const
{
	return RowCount;
}

void SImTableView::AddColumn_Internal(const FName& ColumnId, const FStringView& ColumnToolTip, const FSlateIMSlotData& SlotData)
{
	SHeaderRow::FColumn::FArguments Args = SHeaderRow::FColumn::FArguments()
		.ColumnId(ColumnId)
		.DefaultTooltip(FText::FromStringView(ColumnToolTip))
		.HAlignHeader(HAlign_Fill)
		.VAlignHeader(VAlign_Fill)
		.HeaderContentPadding(SlotData.Padding);

	if (SlotData.bAutoSize && SlotData.MinWidth > 0)
	{
		if (SlotData.MaxWidth > 0 && SlotData.MaxWidth <= SlotData.MinWidth)
		{
			Args.FixedWidth(SlotData.MinWidth);
		}
		else
		{
			Args.ManualWidth(SlotData.MinWidth);
		}
	}
	else
	{
		if (SlotData.MinWidth > 0)
		{
			Args.FillSized(SlotData.MinWidth);
		}
		else
		{
			Args.FillWidth(1.f);
		}
	}
	
	HeaderRow->AddColumn(Args);
	RequestRefresh();
}

void SImTableView::HandleOnExpansionChanged(TSharedRef<FSlateIMTableRow> InItem, bool bInExpanded)
{
	if (TSharedPtr<ISlateIMContainer> Parent = InItem->GetParent())
	{
		if (Parent->IsA<FSlateIMTableBody>())
		{
			StaticCastSharedPtr<FSlateIMTableBody>(Parent)->SetSavedExpansionState(InItem->GetRowId(), bInExpanded);
		}
		else
		{
			StaticCastSharedPtr<FSlateIMTableRow>(Parent)->SetSavedExpansionState(InItem->GetRowId(), bInExpanded);
		}
	}
}

void SImTableHeader::Construct(const SHeaderRow::FArguments& InArgs, const TSharedRef<FSlateIMTableHeader> InHeader)
{
	Header = InHeader;

	SHeaderRow::Construct(InArgs);
}

TSharedPtr<FSlateIMTableHeader> SImTableHeader::GetHeader() const
{
	return Header;
}
