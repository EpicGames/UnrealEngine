// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "ISlateIMContainer.h"
#include "Misc/ISlateIMChild.h"
#include "Widgets/Views/IItemsSource.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"

class SImTableView;

/**
 * Header
 */
class FSlateIMTableHeader : public ISlateIMContainer, public ISlateIMChild, public TSharedFromThis<FSlateIMTableHeader>
{
	SLATE_IM_TYPE_DATA(FSlateIMTableHeader, ISlateIMChild)
	// Deadly diamond of death, but it's okay since ISlateIMContainer and ISlateIMChild don't have implementations for ISlateIMTypeChecking
	// SLATE_IM_TYPE_DATA(FSlateIMTableHeader, ISlateIMContainer)

public:
	virtual FString GetDebugName() override { return GetTypeId().ToString(); }
	virtual int32 GetNumChildren() override { return Children.Num(); }
	virtual FSlateIMChild GetChild(int32 Index) override { return Children.IsValidIndex(Index) ? Children[Index] : nullptr; }
	virtual FSlateIMChild GetContainer() override { return TSharedRef<ISlateIMChild>(AsShared()); }
	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;
	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;

	virtual TSharedPtr<SWidget> GetAsWidget() override;

	TSharedPtr<SImTableView> GetOwningTable() const;
	void SetOwningTable(const TSharedPtr<SImTableView>& InOwningTable);

private:
	TArray<FSlateIMChild> Children;
	TSharedPtr<SImTableView> OwningTable;
};

class SImTableHeader : public SHeaderRow
{
	SLATE_DECLARE_WIDGET(SImTableHeader, SHeaderRow)

public:
	void Construct(const SHeaderRow::FArguments& InArgs, const TSharedRef<FSlateIMTableHeader> InHeader);

	TSharedPtr<FSlateIMTableHeader> GetHeader() const;

private:
	TSharedPtr<FSlateIMTableHeader> Header;
};


class FSlateIMRowExpansionStateData
{
public:
	bool HasSavedExpansionState(uint32 InRowId) const;
	bool GetSavedExpansionState(uint32 InRowId, bool bDefaultState = true) const;
	void SetSavedExpansionState(uint32 InRowId, bool bState);

private:
	TMap<uint32, bool> ExpansionStates;
};

/**
 * Row
 */
class FSlateIMTableRow : public ISlateIMContainer, public ISlateIMChild, public FSlateIMRowExpansionStateData, public TSharedFromThis<FSlateIMTableRow>
{
	SLATE_IM_TYPE_DATA(FSlateIMTableRow, ISlateIMChild)
	// Deadly diamond of death, but it's okay since ISlateIMContainer and ISlateIMChild don't have implementations for ISlateIMTypeChecking
	// SLATE_IM_TYPE_DATA(FSlateIMTableRow, ISlateIMContainer)

public:
	virtual FString GetDebugName() override { return GetTypeId().ToString(); }
	virtual int32 GetNumChildren() override { return Children.Num(); }
	virtual FSlateIMChild GetChild(int32 Index) override { return Children.IsValidIndex(Index) ? Children[Index] : nullptr; }
	virtual FSlateIMChild GetContainer() override { return TSharedRef<ISlateIMChild>(AsShared()); }
	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;
	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
	virtual TSharedPtr<SWidget> GetAsWidget() override;

	uint32 GetRowId() const { return RowId; }
	void SetRowId(uint32 InRowId) { RowId = InRowId; }

	void GetChildRows(TArray<TSharedRef<FSlateIMTableRow>>& OutRows) const;

	TSharedPtr<ISlateIMContainer> GetParent() const;
	void SetParent(TSharedPtr<ISlateIMContainer> InParent);

	int32 CountCellWidgetsUpToIndex(int32 Index) const;
	int32 FindColumnIndex(const FName& ColumnName) const;

	TSharedRef<SWidget> GetCellWidget(int32 CellIndex) const;

	int32 GetColumnCount() const { return ColumnCount; }
	void UpdateColumnCount(int32 NewColumnCount);

	TSharedPtr<SImTableView> GetOwningTable() const;
	void SetOwningTable(const TSharedPtr<SImTableView>& InOwningTable);

	bool IsExpanded();
	bool HasChildRows() const;
	bool ShouldDisplayExpander() const;
	bool AreTableRowContentsRequired();

private:
	TWeakPtr<ISlateIMContainer> Parent;
	uint32 RowId = 0;
	TArray<FSlateIMChild> Children;
	int32 ColumnCount = 0;
	TSharedPtr<SImTableView> OwningTable;
};

class SImTableRow : public SMultiColumnTableRow<TSharedRef<FSlateIMTableRow>>
{
	SLATE_DECLARE_WIDGET(SImTableRow, SMultiColumnTableRow<TSharedRef<FSlateIMTableRow>>)
	
public:	
	void Construct(const FSuperRowType::FArguments& InArgs, const TSharedRef<SImTableView>& InOwnerTableView, const TSharedRef<FSlateIMTableRow>& InTableRow);
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	
private:
	TSharedPtr<FSlateIMTableRow> TableRow;
};



/**
 * Body
 */
class FSlateIMTableBody : public ISlateIMContainer, public ISlateIMChild, public FSlateIMRowExpansionStateData, public TSharedFromThis<FSlateIMTableBody>
{
	SLATE_IM_TYPE_DATA(FSlateIMTableBody, ISlateIMChild)
	// Deadly diamond of death, but it's okay since ISlateIMContainer and ISlateIMChild don't have implementations for ISlateIMTypeChecking
	// SLATE_IM_TYPE_DATA(FSlateIMTableBody, ISlateIMContainer)

public:
	virtual FString GetDebugName() override { return GetTypeId().ToString(); }
	virtual int32 GetNumChildren() override { return Children.Num(); }
	virtual FSlateIMChild GetChild(int32 Index) override { return Children.IsValidIndex(Index) ? FSlateIMChild(Children[Index]) : nullptr; }
	virtual FSlateIMChild GetContainer() override { return TSharedRef<ISlateIMChild>(AsShared()); }
	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;
	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
	virtual TSharedPtr<SWidget> GetAsWidget() override { return nullptr; }

	TSharedPtr<SImTableView> GetOwningTable() const;
	void SetOwningTable(const TSharedPtr<SImTableView>& InOwningTable);

	TSharedPtr<FSlateIMTableRow> GetRow(int32 Index);

	bool IsTree() const;

	const TArray<TSharedRef<FSlateIMTableRow>>& GetTableRows() const;

private:
	TArray<TSharedRef<FSlateIMTableRow>> Children;
	TSharedPtr<SImTableView> OwningTable;
};

// No direct widget representation



/**
 * Table
 */
class SImTableView : public ISlateIMContainer, public STreeView<TSharedRef<FSlateIMTableRow>>
{
	SLATE_DECLARE_WIDGET(SImTableView, STreeView<TSharedRef<FSlateIMTableRow>>)
	SLATE_IM_TYPE_DATA(SImTableView, ISlateIMContainer)
	
public:
	void Construct(const FArguments& InArgs);

	virtual int32 GetNumChildren() override { return 2; }
	virtual FSlateIMChild GetChild(int32 Index) override;
	virtual FSlateIMChild GetContainer() override { return AsShared(); }
	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;
	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
	
	virtual float GetNumLiveWidgets() const override;

	TSharedPtr<FSlateIMTableHeader> GetHeader() const;
	TSharedPtr<FSlateIMTableBody> GetBody() const;

	void AddColumn(const FName& ColumnID, const FStringView& ColumnToolTip, const FSlateIMSlotData& SlotData);

	int32 GetColumnCount() const { return ColumnCount; }

	void EndColumnUpdates();

	void BeginTableUpdates();
	void EndTableUpdates();

	void BeginTableContent();

	void UpdateColumns();

	void SetTableRowStyle(const FTableRowStyle* InRowStyle);

	// A table is a tree if any of its rows has children
	bool IsTree() const;

	void RequestRefresh();
	void RequestRebuild();

	TSharedRef<ITableRow> GenerateRow(TSharedRef<FSlateIMTableRow> InTableRow, const TSharedRef<STableViewBase>& OwnerTable);

	void GatherChildren(TSharedRef<FSlateIMTableRow> Row, TArray<TSharedRef<FSlateIMTableRow>>& OutChildren) const;

	TSharedPtr<FSlateIMTableRow> GetRow(int32 Index);

	int32 GetRowLinearizedIndex(const TSharedRef<FSlateIMTableRow>& Row) const;

	bool IsSameColumnCount(const TSharedRef<FSlateIMTableRow>& Row) const;
	int32 FindColumnIndex(const FName& ColumnName) const;

	void OnRowAdded();
	int32 GetRowCount() const;

private:
	void AddColumn_Internal(const FName& ColumnId, const FStringView& ColumnToolTip, const FSlateIMSlotData& SlotData);

	void HandleOnExpansionChanged(TSharedRef<FSlateIMTableRow> InItem, bool bInExpanded);

	// Because the row source is now in a different object which can change, use this intermediary object 
	// to avoid issues updating the source.
	struct FRowSource : public UE::Slate::ItemsSource::IItemsSource<TSharedRef<FSlateIMTableRow>>
	{
		SImTableView& OwningTable;

		FRowSource(SImTableView& InOwningTable)
			: OwningTable(InOwningTable)
		{}

		virtual const TArrayView<const TSharedRef<FSlateIMTableRow>> GetItems() const override
		{
			if (TSharedPtr<FSlateIMTableBody> BodyLocal = OwningTable.GetBody())
			{
				return BodyLocal->GetTableRows();
			}

			return {};
		}

		virtual bool IsSame(const void* RawPointer) const override
		{
			return this == RawPointer;
		}
	};

	TSharedPtr<FSlateIMTableHeader> Header;
	TSharedPtr<FSlateIMTableBody> Body;
	const FTableRowStyle* RowStyle = nullptr;
	int32 ColumnCount = 0;
	int32 RowCount = 0;

	mutable int32 CachedNumLiveWidgets = 0;

	enum
	{
		Clean,
		NeedsRefresh,
		NeedsRebuild
	} DirtyState = NeedsRebuild;
};
