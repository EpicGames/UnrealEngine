// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMContainer.h"
#include "Widgets/Layout/SWrapBox.h"


class SImWrapBox : public SWrapBox, public ISlateIMContainer
{
	SLATE_DECLARE_WIDGET(SImWrapBox, SWrapBox)
	SLATE_IM_TYPE_DATA(SImWrapBox, ISlateIMContainer)
	
public:
	virtual int32 GetNumChildren() override;
	virtual FSlateIMChild GetChild(int32 Index) override;

	virtual FSlateIMChild GetContainer() override
	{
		return AsShared();
	}

	virtual void RemoveUnusedChildren(int32 LastUsedChildIndex) override;

	virtual void UpdateChild(FSlateIMChild Child, int32 Index, const FSlateIMSlotData& AlignmentData) override;
};
