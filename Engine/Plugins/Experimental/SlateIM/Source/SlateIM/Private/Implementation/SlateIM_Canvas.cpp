// Copyright Epic Games, Inc. All Rights Reserved.

// All implemented functions are WITH_ENGINE
#if WITH_ENGINE

#include "SlateIM.h"
#include "SlateIM_Internal.h"

#include "Containers/SImCompoundWidget.h"
#include "Engine/Canvas.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Implementation/SlateIMEngineCanvasDrawCommandList.h"
#include "Math/IntPoint.h"
#include "Misc/SlateIMManager.h"
#include "Misc/SlateIMSlateResources.h"
#include "SlateMaterialBrush.h"
#include "Styling/SlateColor.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"

namespace SlateIM::Canvas
{
	static const FLazyName CanvasContainerTag = "Canvas";

	const FSlateBrush* GetCanvasBrush(const TSharedRef<SImage>& CanvasImage, TWeakObjectPtr<UObject> WorldContextWeak, int32 Width, int32 Height)
	{
		// Create canvas render target
		UCanvasRenderTarget2D* CanvasRenderTarget = nullptr;
		TSharedPtr<FSlateIMPinnedImageResourceDataStore> CanvasResource = CanvasImage->GetMetaData<FSlateIMPinnedImageResourceDataStore>();

		if (!CanvasResource)
		{
			CanvasRenderTarget = UCanvasRenderTarget2D::CreateCanvasRenderTarget2D(
				WorldContextWeak.Get(),
				UCanvasRenderTarget2D::StaticClass(),
				Width,
				Height
			);

			if (!CanvasRenderTarget)
			{
				return nullptr;
			}

			FSlateBrush Brush;
			Brush.SetResourceObject(CanvasRenderTarget);
			Brush.ImageSize = FVector2D(Width, Height);

			CanvasResource = MakeShared<FSlateIMPinnedImageResourceDataStore>(FPinnedImageResource{ Brush, TStrongObjectPtr<UObject>(CanvasRenderTarget) });
			CanvasImage->AddMetadata(CanvasResource.ToSharedRef());
		}
		else
		{
			CanvasRenderTarget = Cast<UCanvasRenderTarget2D>(CanvasResource->Data.PinnedResource.Get());

			if (!CanvasRenderTarget)
			{
				return nullptr;
			}

			FIntPoint CurrentSize;
			CanvasRenderTarget->GetSize(CurrentSize.X, CurrentSize.Y);

			if (CurrentSize.X != Width || CurrentSize.Y != Height)
			{
				CanvasRenderTarget->ResizeTarget(Width, Height);
				CanvasResource->Data.Brush.SetImageSize(FVector2D(Width, Height));
			}
		}

		// Create uobject proxy for queueing draw calls
		USlateIMEngineCanvasDrawCommandList* UpdateContainer = nullptr;
		TSharedPtr<FSlateIMPinnedObjectResourceDataStore> UpdateResource = CanvasImage->GetMetaData<FSlateIMPinnedObjectResourceDataStore>();

		if (!UpdateResource)
		{
			UpdateContainer = NewObject<USlateIMEngineCanvasDrawCommandList>();
			UpdateResource = MakeShared<FSlateIMPinnedObjectResourceDataStore>(FPinnedObjectResource{ TStrongObjectPtr<UObject>(UpdateContainer) });
			CanvasImage->AddMetadata(UpdateResource.ToSharedRef());
		}
		else
		{
			UpdateContainer = CastChecked<USlateIMEngineCanvasDrawCommandList>(UpdateResource->Data.PinnedResource.Get());
		}

		CanvasRenderTarget->OnCanvasRenderTargetUpdate.Clear();
		CanvasRenderTarget->OnCanvasRenderTargetUpdate.AddDynamic(UpdateContainer, &USlateIMEngineCanvasDrawCommandList::ProcessCommands);

		// Defer update call to EndCanvas

		return &CanvasResource->Data.Brush;
	}

	USlateIMEngineCanvasDrawCommandList* GetCanvasCommandList()
	{
		TSharedPtr<SImCompoundWidget> CanvasContainer = FSlateIMManager::Get().GetCurrentContainer<SImCompoundWidget>();

		if (CanvasContainer && CanvasContainer->GetTag() == CanvasContainerTag)
		{
			if (TSharedPtr<SImage> CanvasImage = StaticCastSharedPtr<SImage>(CanvasContainer->GetChild(0).GetWidget()))
			{
				if (TSharedPtr<FSlateIMPinnedObjectResourceDataStore> CanvasResource = CanvasImage->GetMetaData<FSlateIMPinnedObjectResourceDataStore>())
				{
					return CastChecked<USlateIMEngineCanvasDrawCommandList>(CanvasResource->Data.PinnedResource.Get());
				}
			}
		}

		ensureAlwaysMsgf(false, TEXT("Current container should be a canvas - Is there a missing SlateIM::Canvas::BeginCanvas() statement?"));

		return nullptr;
	}

	void BeginCanvas(int32 Width, int32 Height, const FCanvasParams& CanvasParams)
	{
		BeginContainer();

		// Another class isn't needed here. Tag the container for future checking.
		TSharedPtr<SImCompoundWidget> CanvasContainer = FSlateIMManager::Get().GetCurrentContainer<SImCompoundWidget>();
		CanvasContainer->SetTag(CanvasContainerTag);
		CanvasContainer->SetBackgroundImage(nullptr);
		CanvasContainer->SetAbsorbMouse(false);

		auto GetBrush = [WorldContextWeak = TWeakObjectPtr<UObject>(CanvasParams.WorldContext), Width, Height]
			(const TSharedRef<SImage>& ImageWidget) -> const FSlateBrush*
			{
				return GetCanvasBrush(ImageWidget, WorldContextWeak, Width, Height);
			};

		Image_Internal({.ColorAndOpacity = CanvasParams.ColorAndOpacity, .DesiredSize = CanvasParams.DesiredSize}, GetBrush);

		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->SetUpdateType(CanvasParams.UpdateType);
		}
	}

	void EndCanvas()
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			if (!CommandList->NeedsUpdate())
			{
				EndContainer();
				return;
			}
		}

		TSharedPtr<SImCompoundWidget> CanvasContainer = FSlateIMManager::Get().GetCurrentContainer<SImCompoundWidget>();

		if (CanvasContainer && CanvasContainer->GetTag() == CanvasContainerTag)
		{
			bool bUpdated = false;

			if (TSharedPtr<SImage> CanvasImage = StaticCastSharedPtr<SImage>(CanvasContainer->GetChild(0).GetWidget()))
			{
				if (TSharedPtr<FSlateIMPinnedImageResourceDataStore> CanvasResource = CanvasImage->GetMetaData<FSlateIMPinnedImageResourceDataStore>())
				{
					if (UCanvasRenderTarget2D* CanvasRenderTarget = CastChecked<UCanvasRenderTarget2D>(CanvasResource->Data.PinnedResource.Get()))
					{
						CanvasRenderTarget->UpdateResourceWithParams(UTexture::EUpdateResourceFlags::None);
						bUpdated = true;
					}
				}
			}

			if (!bUpdated)
			{
				ensureAlwaysMsgf(false, TEXT("Internal error."));
			}

			EndContainer();
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Current container should be a canvas - Is there a missing SlateIM::Canvas::BeginCanvas() statement?"));
		}
	}

	void Invalidate()
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->Invalidate();
		}
	}

	void SetClip(const FVector2f& ClipPosition)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[ClipPosition](UCanvas* InCanvas)
				{
					InCanvas->SetClip(ClipPosition.X, ClipPosition.Y);
				}
			));
		}
	}

	void SetDrawColor(const FLinearColor& Color)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[Color](UCanvas* InCanvas)
				{
					InCanvas->SetLinearDrawColor(Color);
				}
			));
		}
	}

	void SetDrawColor(const FColor& Color)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[Color](UCanvas* InCanvas)
				{
					InCanvas->SetDrawColor(Color);
				}
			));
		}
	}

	void DrawItem(TSharedRef<FCanvasItem> ItemPtr, const FVector2f& CanvasPosition)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[ItemPtr, CanvasPosition](UCanvas* InCanvas)
				{
					InCanvas->DrawItem(*ItemPtr, CanvasPosition.X, CanvasPosition.Y);
				}
			));
		}
	}

	void DrawIcon(const FCanvasIcon& Icon, const FVector2f& CanvasPosition, const FVector2f& Scale)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[Icon, CanvasPosition, Scale](UCanvas* InCanvas)
				{
					InCanvas->DrawScaledIcon(
						Icon, 
						CanvasPosition.X, 
						CanvasPosition.Y, 
						FVector(Scale.X, Scale.Y, 1)
					);
				}
			));
		}
	}

	void DrawLine(const FVector2D& CanvasPositionA, const FVector2D& CanvasPositionB, float Thickness, const FLinearColor& RenderColor)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[CanvasPositionA, CanvasPositionB, Thickness, RenderColor](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawLine(
						CanvasPositionA, 
						CanvasPositionB, 
						Thickness,
						RenderColor
					);
				}
			));
		}
	}

	void DrawTexture(UTexture* RenderTexture, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FTileRenderParams& RenderParams)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderTexture, CanvasPosition, CanvasSize, RenderParams](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawTexture(
						RenderTexture, 
						CanvasPosition,
						CanvasSize,
						RenderParams.UVPosition,
						RenderParams.UVSize,
						RenderParams.RenderColor,
						RenderParams.BlendMode,
						RenderParams.Rotation,
						RenderParams.PivotPoint
					);
				}
			));
		}
	}

	void DrawMaterial(UMaterialInterface* RenderMaterial, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FTileRenderParams& RenderParams)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderMaterial, CanvasPosition, CanvasSize, RenderParams](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawMaterial(
						RenderMaterial,
						CanvasPosition,
						CanvasSize,
						RenderParams.UVPosition,
						RenderParams.UVSize,
						RenderParams.Rotation,
						RenderParams.PivotPoint
					);
				}
			));
		}
	}

	void DrawText(UFont* RenderFont, const FString& RenderText, const FVector2D& CanvasPosition, const FTextRenderParams& RenderParams)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderFont, RenderText, CanvasPosition, RenderParams](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawText(
						RenderFont,
						RenderText,
						CanvasPosition,
						RenderParams.Scale,
						RenderParams.RenderColor,
						RenderParams.Kerning,
						RenderParams.ShadowColor,
						RenderParams.ShadowOffset,
						RenderParams.bCentreX,
						RenderParams.bCentreY,
						RenderParams.bOutlined,
						RenderParams.OutlineColor
					);
				}
			));
		}
	}

	void DrawText(UFont* RenderFont, const FString& RenderText, const FVector2f& CanvasPosition, const FVector2f& Scale, const FFontRenderInfo& RenderInfo)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderFont, RenderText, CanvasPosition, Scale, RenderInfo](UCanvas* InCanvas)
				{
					InCanvas->DrawText(
						RenderFont,
						RenderText,
						CanvasPosition.X,
						CanvasPosition.Y,
						Scale.X,
						Scale.Y,
						RenderInfo
					);
				}
			));
		}
	}

	void DrawWrappedText(const UFont* RenderFont, const FString& RenderText, const FVector2f& CanvasPosition, bool bCenterTextX, bool bCenterTextY, const FVector2f& Scale, const FFontRenderInfo& RenderInfo)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderFont, RenderText, CanvasPosition, Scale, bCenterTextX, bCenterTextY, RenderInfo](UCanvas* InCanvas)
				{
					int32 outXL, outYL;

					InCanvas->WrappedPrint(
						/* Draw */ true,
						CanvasPosition.X,
						CanvasPosition.Y,
						outXL,
						outYL,
						RenderFont,
						Scale.X,
						Scale.Y,
						bCenterTextX,
						bCenterTextY,
						RenderText,
						RenderInfo
					);
				}
			));
		}
	}

	void DrawBorder(UTexture* BorderTexture, UTexture* BackgroundTexture, UTexture* LeftBorderTexture, UTexture* RightBorderTexture, UTexture* TopBorderTexture, UTexture* BottomBorderTexture, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FBorderRenderParams& RenderParams)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[BorderTexture,BackgroundTexture, LeftBorderTexture, RightBorderTexture, TopBorderTexture, BottomBorderTexture, 
					CanvasPosition, CanvasSize, RenderParams](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawBorder(
						BorderTexture,
						BackgroundTexture,
						LeftBorderTexture,
						RightBorderTexture,
						TopBorderTexture,
						BottomBorderTexture,
						CanvasPosition,
						CanvasSize,
						RenderParams.UVPosition,
						RenderParams.UVSize,
						RenderParams.RenderColor,
						RenderParams.BorderScale,
						RenderParams.BackgroundScale,
						RenderParams.Rotation,
						RenderParams.PivotPoint,
						RenderParams.CornerSize
					);
				}
			));
		}
	}

	void DrawBox(const FVector2D& CanvasPosition, const FVector2D& CanvasSize, float Thickness, FLinearColor RenderColor)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[CanvasPosition, CanvasSize, Thickness, RenderColor](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawBox(
						CanvasPosition,
						CanvasSize,
						Thickness,
						RenderColor
					);
				}
			));
		}
	}

	void DrawTriangles(UTexture* RenderTexture, TSharedRef<TArray<FCanvasUVTri>> Triangles)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderTexture, Triangles](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawTriangle(RenderTexture, *Triangles);
				}
			));
		}
	}

	void DrawMaterialTriangles(UMaterialInterface* RenderMaterial, TSharedRef<TArray<FCanvasUVTri>> Triangles)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderMaterial, Triangles](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawMaterialTriangle(RenderMaterial, *Triangles);
				}
			));
		}
	}

	void DrawPolygon(UTexture* RenderTexture, const FVector2D& CanvasPosition, const FVector2D& Radius, int32 NumberOfSides, FLinearColor RenderColor)
	{
		if (USlateIMEngineCanvasDrawCommandList* CommandList = GetCanvasCommandList())
		{
			CommandList->EnqueueCommand(USlateIMEngineCanvasDrawCommandList::FCanvasDrawCommand::CreateLambda(
				[RenderTexture, CanvasPosition, Radius, NumberOfSides, RenderColor](UCanvas* InCanvas)
				{
					InCanvas->K2_DrawPolygon(
						RenderTexture, 
						CanvasPosition,
						Radius, 
						NumberOfSides, 
						RenderColor
					);
				}
			));
		}
	}
} // SlateIM::Canvas

#endif // WITH_ENGINE
