// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Containers/SImCompoundWidget.h"
#include "Containers/SImContextMenuAnchor.h"
#include "Containers/SImPopUp.h"
#include "Containers/SImScrollBox.h"
#include "Containers/SImStackBox.h"
#include "Containers/SImTableView.h"
#include "Containers/SImWrapBox.h"
#include "Misc/SlateIMLogging.h"
#include "Misc/SlateIMManager.h"
#include "Misc/SlateIMWidgetScope.h"

namespace SlateIM
{
	void BeginContainer()
	{
		TSharedPtr<SImCompoundWidget> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImCompoundWidget> Scope;
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				ContainerWidget = SNew(SImCompoundWidget);
				Scope.UpdateWidget(ContainerWidget);
			}
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
	}

	void EndContainer()
	{
		FSlateIMManager::Get().PopContainer<SImCompoundWidget>();
	}

	void BeginStack(EOrientation Orientation)
	{
		TSharedPtr<SImStackBox> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImStackBox> Scope;
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				ContainerWidget = SNew(SImStackBox);
				Scope.UpdateWidget(ContainerWidget);
			}

			ContainerWidget->SetOrientation(Orientation);
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
	}

	void EndStack()
	{
		FSlateIMManager::Get().PopContainer<SImStackBox>();
	}

	void BeginWrap(EOrientation Orientation)
	{
		TSharedPtr<SImWrapBox> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImWrapBox> Scope;
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				// Note, SNew cannot take a template as the argument because it converts the argument into a string for type identification. So we have to hard code this even though
				// its essentially the same as BeginHorizonal, just with a different type
				ContainerWidget = SNew(SImWrapBox);
				Scope.UpdateWidget(ContainerWidget);
			}

			ContainerWidget->SetUseAllottedSize(true);
			ContainerWidget->SetOrientation(Orientation);
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
	}

	void EndWrap()
	{
		FSlateIMManager::Get().PopContainer<SImWrapBox>();
	}

	void BeginHorizontalStack(bool bMaximizeContent)
	{
		if (bMaximizeContent)
		{
			SlateIM::Maximize();
		}

		BeginStack(Orient_Horizontal);
	}

	void EndHorizontalStack()
	{
		EndStack();
	}

	void BeginVerticalStack(bool bMaximizeContent)
	{
		if (bMaximizeContent)
		{
			SlateIM::Maximize();
		}

		BeginStack(Orient_Vertical);
	}

	void EndVerticalStack()
	{
		EndStack();
	}

	void BeginHorizontalWrap(bool bMaximizeContent)
	{
		if (bMaximizeContent)
		{
			SlateIM::Maximize();
		}

		BeginWrap(Orient_Horizontal);
	}

	void EndHorizontalWrap()
	{
		EndWrap();
	}

	void BeginVerticalWrap(bool bMaximizeContent)
	{
		if (bMaximizeContent)
		{
			SlateIM::Maximize();
		}

		BeginWrap(Orient_Vertical);
	}

	void EndVerticalWrap()
	{
		EndWrap();
	}

	void BeginBorder(const FSlateBrush* BackgroundImage, EOrientation Orientation, bool bAbsorbMouse, FMargin ContentPadding)
	{
		BeginBorder(BackgroundImage, {.Orientation = Orientation, .bAbsorbMouse = bAbsorbMouse, .ContentPadding = ContentPadding});
	}

	void BeginBorder(const FSlateBrush* BackgroundImage, const FBorderParams& Params)
	{
		TSharedPtr<SImCompoundWidget> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImCompoundWidget> Scope(Params.ContentPadding, HAlign_Fill, VAlign_Fill, false);
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				ContainerWidget = SNew(SImCompoundWidget);
					
				Scope.UpdateWidget(ContainerWidget);
			}


			ContainerWidget->SetBackgroundImage(BackgroundImage);
			ContainerWidget->SetContentPadding(Params.ContentPadding);
			ContainerWidget->SetAbsorbMouse(Params.bAbsorbMouse);
			ContainerWidget->SetOrientation(Params.Orientation);
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
	}

	void BeginBorder(const FName BorderStyleName, EOrientation Orientation, bool bAbsorbMouse, FMargin ContentPadding)
	{
		BeginBorder(FAppStyle::GetBrush(BorderStyleName), {.Orientation = Orientation, .bAbsorbMouse = bAbsorbMouse, .ContentPadding = ContentPadding});
	}

	void BeginBorder(const FName BorderStyleName, const FBorderParams& Params)
	{
		BeginBorder(FAppStyle::GetBrush(BorderStyleName), Params);
	}

	void EndBorder()
	{
		FSlateIMManager::Get().PopContainer<SImCompoundWidget>();
	}

	bool BeginScrollBox(EOrientation Orientation)
	{
		return BeginScrollBox({.Orientation = Orientation});
	}

	bool BeginScrollBox(const FScrollBoxParams& Params)
	{
		TSharedPtr<SImScrollBox> ContainerWidget;

		bool bUserScrolled = false;
		{
			FWidgetScope<SImScrollBox> Scope(false);
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				ContainerWidget =
					SNew(SImScrollBox)
					.Orientation(Params.Orientation)
					.OnUserScrolled_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()](float Offset)
					{
						FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
					});

				Scope.UpdateWidget(ContainerWidget);
			}
			else
			{
				bUserScrolled = Scope.IsActivatedThisFrame();
			}
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));

		return bUserScrolled;
	}

	void EndScrollBox()
	{
		FSlateIMManager::Get().PopContainer<SImScrollBox>();
	}

	void BeginPopUp(const FName BorderStyleName, const FPopUpParams& Params)
	{
		BeginPopUp(FAppStyle::GetBrush(BorderStyleName), Params);
	}

	void BeginPopUp(const FName BorderStyleName, EOrientation Orientation, bool bAbsorbMouse, FMargin ContentPadding)
	{
		BeginPopUp(FAppStyle::GetBrush(BorderStyleName), {.Orientation = Orientation, .bAbsorbMouse = bAbsorbMouse, .ContentPadding = ContentPadding});
	}

	void BeginPopUp(const FSlateBrush* BorderBrush, EOrientation Orientation, bool bAbsorbMouse, FMargin ContentPadding)
	{
		BeginPopUp(BorderBrush, {.Orientation = Orientation, .bAbsorbMouse = bAbsorbMouse, .ContentPadding = ContentPadding});
	}

	void BeginPopUp(const FSlateBrush* BorderBrush, const FPopUpParams& Params)
	{
		TSharedPtr<SImPopUp> ContainerWidget;
		{
			FWidgetScope<SImPopUp> Scope(FMargin(0));
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				ContainerWidget =
					SNew(SImPopUp)
					.ShowMenuBackground(false); 
				
				Scope.UpdateWidget(ContainerWidget);
			}

			// Setting focus by default will cause the pop-up to auto-close when it loses focus which would mean we couldn't have multiple pop-ups at once
			// nor could we open a pop-up in response to something else getting focus, so we don't focus the pop-up
			constexpr bool bIsOpen = true;
			constexpr bool bSetFocus = false;
			ContainerWidget->SetIsOpen(bIsOpen, bSetFocus);
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
		SlateIM::BeginBorder(BorderBrush, Params.Orientation, Params.bAbsorbMouse, Params.ContentPadding);
	}

	void EndPopUp()
	{
		SlateIM::EndBorder();
		FSlateIMManager::Get().PopContainer<SImPopUp>();
	}

	void BeginTableRow()
	{
		// If we're in a row, we're adding a child row, otherwise it's a top-level row in the body
		TSharedPtr<FSlateIMTableBody> Body = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableBody>();
		TSharedPtr<FSlateIMTableRow> ParentRow = !Body.IsValid() ? FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>() : nullptr;
		if (ensureMsgf(Body || ParentRow, TEXT("Table Rows/Cells can only exist within Table Bodies")))
		{
			TSharedPtr<SImTableView> Table;

			if (Body)
			{
				Table = Body->GetOwningTable();
			}
			else if (ParentRow)
			{
				Table = ParentRow->GetOwningTable();
			}

			TSharedPtr<FSlateIMTableRow> TableRow;
			// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
			{
				FWidgetScope<FSlateIMTableRow> Scope;
				TableRow = Scope.GetWidget();

				if (!TableRow)
				{
					TableRow = MakeShared<FSlateIMTableRow>();
					Scope.UpdateWidget(TableRow);
				}
				else if (Table && !Table->IsSameColumnCount(TableRow.ToSharedRef()))
				{
					Scope.UpdateWidget(TableRow);
				}

				if (ParentRow)
				{
					TableRow->SetParent(ParentRow);
				}
				else
				{
					TableRow->SetParent(Body);
				}

				if (Table)
				{
					Table->OnRowAdded();
				}
			}

			FSlateIMManager::Get().PushContainer(FContainerNode(TableRow));
		}
	}

	void EndTableRow()
	{
		FSlateIMManager::Get().PopContainer<FSlateIMTableRow>();
	}

	void BeginTableColumn(const FName& ColumnID)
	{
		if (TSharedPtr<FSlateIMTableHeader> Header = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableHeader>())
		{
			if (TSharedPtr<SImTableView> Table = Header->GetOwningTable())
			{
				constexpr bool bAutoSize = false;
				const FSlateIMSlotData SlotData = FSlateIMManager::Get().GetCurrentAlignmentData(Defaults::Padding, HAlign_Fill, VAlign_Fill, bAutoSize
					, Defaults::MinWidth, Defaults::MinHeight, Defaults::MaxWidth, Defaults::MaxHeight);

				Table->AddColumn(ColumnID, FSlateIMManager::Get().GetCurrentRoot().CurrentToolTip, SlotData);

				FSlateIMManager::Get().GetMutableCurrentRoot().CurrentToolTip.Empty();
				FSlateIMManager::Get().ResetAlignmentData();

				SlateIM::BeginHorizontalStack();
			}
			else
			{
				ensureAlwaysMsgf(Table, TEXT("Internal error: Table header has not been assigned its owning table."));
			}
		}
		else
		{
			ensureAlwaysMsgf(Header, TEXT("Current container should be a table header - Is there a missing SlateIM::BeginTable() or SlateIM::BeginTableHeader() statement?"));
		}
	}

	void EndTableColumn()
	{
		SlateIM::EndHorizontalStack();
	}

	void BeginTable(const FTableViewStyle* InStyle, const FTableRowStyle* InRowStyle)
	{
		BeginTable({ .Style = InStyle, .RowStyle = InRowStyle });
	}
	
	void BeginTable(const FTableParams& Params)
	{
		TSharedPtr<SImTableView> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImTableView> Scope;
			ContainerWidget = Scope.GetWidget();
			
			Scope.HashData(Params.Style);

			if (!ContainerWidget)
			{
				ContainerWidget = SNew(SImTableView)
					.TreeViewStyle(Params.Style)
					.SelectionMode(Params.SelectionMode);
				Scope.UpdateWidget(ContainerWidget);
			}
			else if (Scope.IsDataHashDirty())
			{
				ContainerWidget->SetStyle(Params.Style);
			}

			ContainerWidget->SetTableRowStyle(Params.RowStyle);
			ContainerWidget->BeginTableUpdates();
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
	}

	void EndTable()
	{
		// If there's an open column or cell, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			// Column
			if (Cell->GetOrientation() == Orient_Horizontal)
			{
				EndTableColumn();

				// Pop header
				if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableHeader>())
				{
					EndTableHeader();
				}
			}
			// Cell
			else if (Cell->GetOrientation() == Orient_Vertical)
			{
				SlateIM::EndVerticalStack();

				// Pop row
				while (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>())
				{
					EndTableRow();
				}

				// Pop body
				if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableBody>())
				{
					EndTableBody();
				}
			}
		}
		// Pop header
		else if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableHeader>())
		{
			EndTableHeader();
		}
		// Pop row
		else if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>())
		{
			while (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>())
			{
				EndTableRow();
			}

			// Pop body
			if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableBody>())
			{
				EndTableBody();
			}
		}
		// Pop body
		else if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableBody>())
		{
			EndTableBody();
		}
		else if (!FSlateIMManager::Get().GetCurrentContainer<SImTableView>())
		{
			ensureAlwaysMsgf(false, TEXT("Current container should be a row, body or a table - Is there a missing SlateIM::EndX() statement?"));
			return;
		}

		TSharedPtr<SImTableView> Table = FSlateIMManager::Get().GetCurrentContainer<SImTableView>();

		if (ensureAlwaysMsgf(Table, TEXT("Current container should be a table - Is there a missing SlateIM::EndX() statement?")))
		{
			FSlateIMManager::Get().PopContainer<SImTableView>();
			Table->EndTableUpdates();
		}
	}

	void BeginTableHeader()
	{
		if (TSharedPtr<SImTableView> Table = FSlateIMManager::Get().GetCurrentContainer<SImTableView>())
		{
			FSlateIMManager::Get().PushContainer(FContainerNode(Table->GetHeader()));
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Current container should be a table - Is there a missing SlateIM::BeginTable() statement?"));
		}
	}

	void EndTableHeader()
	{
		// If there's an open column, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			EndTableColumn();
		}

		FSlateIMManager::Get().PopContainer<FSlateIMTableHeader>();

		if (TSharedPtr<SImTableView> Table = FSlateIMManager::Get().GetCurrentContainer<SImTableView>())
		{
			Table->EndColumnUpdates();
		}
	}

	void NextTableColumn(const FName& ColumnID)
	{
		// If we haven't started the header, do so now.
		if (TSharedPtr<SImTableView> Table = FSlateIMManager::Get().GetCurrentContainer<SImTableView>())
		{
			BeginTableHeader();
		}
		// If there's an open column, close it
		else if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			EndTableColumn();
		}
		else if (!FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableHeader>())
		{
			ensureAlwaysMsgf(false, TEXT("Current container should be a table or a table header - Is there a missing SlateIM::BeginTable() or SlateIM::BeginTableHeader() statement?"));
			return;
		}

		BeginTableColumn(ColumnID);
	}

	void AddTableColumn(const FName& ColumnID, const FStringView& Label)
	{
		AddTableColumn(ColumnID, {.Label = Label});
	}

	void AddTableColumn(const FName& ColumnID, const FTableColumnParams& Params)
	{
		FSlateIMAlignmentState AlignmentState = FSlateIMManager::Get().SaveAlignmentState();

		// These values are used to size the column container and should not affect its contents.
		AlignmentState.MinWidth.Reset();
		AlignmentState.MaxWidth.Reset();
		AlignmentState.MinHeight.Reset();
		AlignmentState.MaxHeight.Reset();

		NextTableColumn(ColumnID);

		if (!Params.Label.IsEmpty())
		{
			FSlateIMManager::Get().RestoreAlignmentState(AlignmentState);
			SlateIM::Text(Params.Label);
		}

		EndTableColumn();
	}

	void FixedTableColumnWidth(float Width)
	{
		SlateIM::AutoSize();
		SlateIM::MinWidth(Width);
		SlateIM::MaxWidth(Width);
	}

	void InitialTableColumnWidth(float Width)
	{
		SlateIM::AutoSize();
		SlateIM::MinWidth(Width);
	}

	void BeginTableBody()
	{
		// If there's an open column, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			EndTableColumn();
		}
		
		// If the header is open, end it
		if (TSharedPtr<FSlateIMTableHeader> Header = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableHeader>())
		{
			EndTableHeader();
		}

		if (TSharedPtr<SImTableView> Table = FSlateIMManager::Get().GetCurrentContainer<SImTableView>())
		{
			FSlateIMManager::Get().PushContainer(FContainerNode(Table->GetBody()));
			Table->BeginTableContent();
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Current container should be a table - Is there a missing SlateIM::BeginTable() statement?"));
		}
	}

	void EndTableBody()
	{
		// If there's an open cell, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			SlateIM::EndVerticalStack();
		}

		// If there's an open row, end it
		if (TSharedPtr<FSlateIMTableRow> Row = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>())
		{
			EndTableRow();
		}

		FSlateIMManager::Get().PopContainer<FSlateIMTableBody>();
	}

	bool NextTableCell(bool* bOutRowSelected)
	{
		SCOPED_NAMED_EVENT_TEXT("SlateIM::NextTableCell", FColorList::Goldenrod);

		// If there's an open column or cell, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			if (Cell->GetOrientation() == Orient_Horizontal)
			{
				SlateIM::EndHorizontalStack();
			}
			else if (Cell->GetOrientation() == Orient_Vertical)
			{ 
				SlateIM::EndVerticalStack();
			}
		}

		// If we haven't ended the header, do that now.
		if (TSharedPtr<FSlateIMTableHeader> Header = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableHeader>())
		{
			EndTableHeader();
		}
		
		// We haven't drawn any table cells yet, begin the first row
		if (TSharedPtr<SImTableView> Table = FSlateIMManager::Get().GetCurrentContainer<SImTableView>())
		{
			BeginTableBody();
			BeginTableRow();
		}
		else if (FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableBody>())
		{
			BeginTableRow();
		}

		TSharedPtr<FSlateIMTableRow> Row = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>();
		if (ensure(Row))
		{
			const FContainerNode* ContainerNode = FSlateIMManager::Get().GetCurrentContainerNode();
			check(ContainerNode);
			
			// Start next row if we've filled all the columns in this row
			if (Row->GetColumnCount() == Row->CountCellWidgetsUpToIndex(ContainerNode->LastUsedChildIndex))
			{
				EndTableRow();
				BeginTableRow();
				Row = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>();
				ensure(Row);
			}

			if (bOutRowSelected)
			{
				if (TSharedPtr<SImTableView> OwningTable = Row->GetOwningTable())
				{
					*bOutRowSelected = OwningTable->IsItemSelected(Row.ToSharedRef());
				}
			}			

			// Default table cells to fill (NextHAlign might have been set by the user, so we check first) 
			if (!FSlateIMManager::Get().NextHAlign.IsSet())
			{
				FSlateIMManager::Get().NextHAlign = HAlign_Fill;
			}
			
			// Create a vertical stack to act as our cell widget
			// TODO - Only automatically create a container when one is not created by the user

			SlateIM::BeginVerticalStack();

			return Row->AreTableRowContentsRequired();
		}

		return false;
	}

	bool BeginTableRowChildren(uint32 ParentRowId, bool bDefaultExpanded)
	{
		return BeginTableRowChildren({.ParentRowId = ParentRowId, .bDefaultExpanded = bDefaultExpanded});
	}
	bool BeginTableRowChildren(const FTableRowChildrenParams& Params)
	{
		// If there's an open cell, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			SlateIM::EndVerticalStack();
		}

		TSharedPtr<FSlateIMTableRow> ParentRow = FSlateIMManager::Get().GetCurrentContainer<FSlateIMTableRow>();
		if (!ensureMsgf(ParentRow, TEXT("Child Table Rows can only be added to table rows. Did you forget to call NextTableCell()?")))
		{
			return false;
		}

		ParentRow->SetRowId(Params.ParentRowId);

		SlateIM::BeginTableRow();

		if (Params.ParentRowId != 0)
		{
			if (TSharedPtr<SImTableView> OwningTable = ParentRow->GetOwningTable())
			{
				// If the default expanded state is true, we have a state because the default is unexpanded.
				bool bHasExpansionState = Params.bDefaultExpanded;
				bool bExpanded = Params.bDefaultExpanded;
				TSharedPtr<ISlateIMContainer> GrandParent = ParentRow->GetParent();

				if (GrandParent->IsA<FSlateIMTableBody>())
				{
					TSharedPtr<FSlateIMTableBody> GrandParentRow = StaticCastSharedPtr<FSlateIMTableBody>(GrandParent);
					
					if (GrandParentRow->HasSavedExpansionState(Params.ParentRowId))
					{
						bHasExpansionState = true;
						bExpanded = GrandParentRow->GetSavedExpansionState(Params.ParentRowId, Params.bDefaultExpanded);
					}
				}
				else
				{
					TSharedPtr<FSlateIMTableRow> GrandParentBody = StaticCastSharedPtr<FSlateIMTableRow>(GrandParent);

					if (GrandParentBody->HasSavedExpansionState(Params.ParentRowId))
					{
						bHasExpansionState = true;
						bExpanded = GrandParentBody->GetSavedExpansionState(Params.ParentRowId, Params.bDefaultExpanded);
					}
				}

				if (bHasExpansionState)
				{
					OwningTable->SetItemExpansion(ParentRow.ToSharedRef(), bExpanded);
					return bExpanded;
				}
			}
		}

		return ParentRow->IsExpanded();
	}

	void EndTableRowChildren()
	{
		// If there's an open cell, close it
		if (TSharedPtr<SImStackBox> Cell = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>())
		{
			SlateIM::EndVerticalStack();
		}
		EndTableRow();
	}
}
