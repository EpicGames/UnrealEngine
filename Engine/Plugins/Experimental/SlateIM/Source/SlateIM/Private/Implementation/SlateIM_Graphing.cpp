// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Misc/SlateIMManager.h"
#include "Misc/SlateIMWidgetScope.h"
#include "Widgets/SImGraph.h"

namespace SlateIM
{
	TSharedPtr<SImGraph> GetCurrentGraphWidget()
	{
		if (TSharedPtr<SWidget> ChildWidget = FSlateIMManager::Get().GetCurrentChildAsWidget())
		{
			if (ChildWidget->GetWidgetClass().GetWidgetType() == SImGraph::StaticWidgetClass().GetWidgetType())
			{
				return StaticCastSharedPtr<SImGraph>(ChildWidget);
			}
		}

		return nullptr;
	}
	
	void BeginGraph()
	{
		FWidgetScope<SImGraph> Scope;
		TSharedPtr<SImGraph> GraphWidget = Scope.GetWidget();

		if (!GraphWidget)
		{
			GraphWidget = SNew(SImGraph);
			Scope.UpdateWidget(GraphWidget);
		}
		
		GraphWidget->BeginGraph();
	}

	void EndGraph()
	{
		TSharedPtr<SImGraph> GraphWidget = GetCurrentGraphWidget();
		if (ensureMsgf(GraphWidget, TEXT("Calling SlateIM::EndGraph() but the last widget is not a graph widget")))
		{
			GraphWidget->EndGraph();
		}
	}

	void GraphLine(const TArrayView<FVector2D>& Points, const FLinearColor& LineColor, float LineThickness, const FDoubleRange& XViewRange, const FDoubleRange& YViewRange)
	{
		GraphLine(Points, {.XViewRange = XViewRange, .YViewRange = YViewRange, .LineColor = LineColor, .LineThickness = LineThickness});
	}

	void GraphLine(const TArrayView<FVector2D>& Points, const FGraphLinePointsParams& Params)
	{
		TSharedPtr<SImGraph> GraphWidget = GetCurrentGraphWidget();
		if (ensureMsgf(GraphWidget, TEXT("Calling SlateIM::GraphLine() but the last widget is not a graph widget")))
		{
			GraphWidget->AddLineGraph(Points, Params);
		}
	}

	void GraphLine(const TArrayView<double>& Values, const FLinearColor& LineColor, float LineThickness, const FDoubleRange& ViewRange)
	{
		GraphLine(Values, {.ViewRange = ViewRange, .LineColor = LineColor, .LineThickness = LineThickness});
	}

	void GraphLine(const TArrayView<double>& Values, const FGraphLineValuesParams& Params)
	{
		TSharedPtr<SImGraph> GraphWidget = GetCurrentGraphWidget();
		if (ensureMsgf(GraphWidget, TEXT("Calling SlateIM::GraphLine() but the last widget is not a graph widget")))
		{
			GraphWidget->AddLineGraph(Values, Params);
		}
	}
}
