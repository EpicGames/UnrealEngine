// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SlateIMInputState.h"
#include "Misc/SlateIMManager.h"
#include "SlateIM.h"

namespace SlateIM
{
	bool IsKeyPressed(const FKey& InKey)
	{
		if (TSharedPtr<ISlateIMRoot> Root = FSlateIMManager::Get().GetCurrentRoot().RootWidget)
		{
			const ESlateIMKeyState* KeyStatePtr = Root->GetInputState().KeyStateMap.Find(InKey);
			return KeyStatePtr != nullptr && *KeyStatePtr == ESlateIMKeyState::Pressed;
		}

		return false;
	}

	bool IsKeyHeld(const FKey& InKey)
	{
		if (TSharedPtr<ISlateIMRoot> Root = FSlateIMManager::Get().GetCurrentRoot().RootWidget)
		{
			const ESlateIMKeyState* KeyStatePtr = Root->GetInputState().KeyStateMap.Find(InKey);
			return KeyStatePtr != nullptr && (*KeyStatePtr == ESlateIMKeyState::Pressed || *KeyStatePtr == ESlateIMKeyState::Held);
		}

		return false;
	}

	bool IsKeyReleased(const FKey& InKey)
	{
		if (TSharedPtr<ISlateIMRoot> Root = FSlateIMManager::Get().GetCurrentRoot().RootWidget)
		{
			const ESlateIMKeyState* KeyStatePtr = Root->GetInputState().KeyStateMap.Find(InKey);
			return KeyStatePtr != nullptr && *KeyStatePtr == ESlateIMKeyState::Released;
		}

		return false;
	}

	float GetKeyAnalogValue(const FKey& InKey)
	{
		if (TSharedPtr<ISlateIMRoot> Root = FSlateIMManager::Get().GetCurrentRoot().RootWidget)
		{
			const float* AnalogValuePtr = Root->GetInputState().AnalogValueMap.Find(InKey);
			return (AnalogValuePtr != nullptr) ? *AnalogValuePtr : 0.f;
		}

		return 0.f;
	}
}
