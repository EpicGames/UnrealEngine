// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Containers/SImContextMenuAnchor.h"
#include "Containers/SImStackBox.h"
#include "Misc/SlateIMManager.h"
#include "Misc/SlateIMWidgetScope.h"
#include "SlateIMStyle.h"

namespace SlateIM
{
	namespace Menu
	{
		const FLazyName MenuBoxTag = "MenuBox";
	}

	bool BeginContextMenuAnchor()
	{
		TSharedPtr<SImContextMenuAnchor> ContainerWidget;

		bool bMenuOpened = false;
		{
			FWidgetScope<SImContextMenuAnchor> Scope;
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget)
			{
				ContainerWidget = SNew(SImContextMenuAnchor);

				Scope.UpdateWidget(ContainerWidget);
			}
			else
			{
				bMenuOpened = ContainerWidget->IsMenuOpen();
			}
		}


		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
		FSlateIMManager::Get().PushMenuRoot(ContainerWidget);

		return bMenuOpened;
	}

	void EndContextMenuAnchor()
	{
		FSlateIMManager::Get().PopMenuRoot();
		FSlateIMManager::Get().PopContainer<SImContextMenuAnchor>();
	}

	void AddMenuSeparator()
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot add menu items without a current active menu anchor"));

		return Anchor->AddMenuSeparator();
	}
	
	void BeginMenuBar()
	{
		SlateIM::Padding(0);
		BeginHorizontalStack();
		
		// Another class isn't needed here. Tag the stackbox for future checking.
		TSharedPtr<SImStackBox> MenuBox = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>();
		MenuBox->SetTag(Menu::MenuBoxTag);
	}

	void EndMenuBar()
	{
		// Pop past open menu if it isn't closed.
		if (TSharedPtr<SImContextMenuAnchor> MenuAnchor = FSlateIMManager::Get().GetCurrentContainer<SImContextMenuAnchor>())
		{
			EndContextMenuAnchor();
		}

		TSharedPtr<SImStackBox> MenuBox = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>();

		if (!ensureAlwaysMsgf(MenuBox && MenuBox->GetTag() == Menu::MenuBoxTag, TEXT("Attempting to close a menu bar when not in a menu.")))
		{
			return;
		}

		EndHorizontalStack();
	}

	void AddMenuBarEntry(const FStringView& InMenuName)
	{
		// Pop past open menu if it isn't closed.
		if (TSharedPtr<SImContextMenuAnchor> MenuAnchor = FSlateIMManager::Get().GetCurrentContainer<SImContextMenuAnchor>())
		{
			EndContextMenuAnchor();
		}

		TSharedPtr<SImStackBox> MenuBox = FSlateIMManager::Get().GetCurrentContainer<SImStackBox>();

		if (!ensureAlwaysMsgf(MenuBox && MenuBox->GetTag() == Menu::MenuBoxTag, TEXT("Attempting to add a menu entry when not in a menu.")))
		{
			return;
		}

		SlateIM::Padding(0);
		BeginContextMenuAnchor();

		SlateIM::Padding(0);

		if (Button(InMenuName, {.Style = &FSlateIMStyle::Get().GetWidgetStyle<FButtonStyle>("SlateIM.MenuButton")}))
		{
			if (TSharedPtr<SImContextMenuAnchor> Menu = FSlateIMManager::Get().GetCurrentMenuRoot())
			{
				FVector2D Position = Menu->GetTickSpaceGeometry().GetAbsolutePosition();
				Position.Y += Menu->GetTickSpaceGeometry().GetAbsoluteSize().Y;

				Menu->OpenMenu(Position);
			}
		}
	}

	void EndMenuBarEntry()
	{
		TSharedPtr<SImContextMenuAnchor> MenuAnchor = FSlateIMManager::Get().GetCurrentContainer<SImContextMenuAnchor>();

		if (!ensureAlwaysMsgf(MenuAnchor, TEXT("Attempting to close a menu when not in a menu.")))
		{
			return;
		}

		EndContextMenuAnchor();
	}

	void NextMenuBarEntry(const FStringView& InMenuName)
	{
		AddMenuBarEntry(InMenuName);
	}

	void AddMenuSection(const FStringView& SectionText)
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot add menu items without a current active menu anchor"));

		return Anchor->AddMenuSection(SectionText);
	}

	bool AddMenuButton(const FStringView& RowText, const FStringView& ToolTipText)
	{
		return AddMenuButton(RowText, {.ToolTipText = ToolTipText});
	}

	bool AddMenuButton(const FStringView& RowText, const FMenuButtonParams& Params)
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot add menu items without a current active menu anchor"));

		return Anchor->AddMenuButton(RowText, Params.ToolTipText);
	}

	bool AddMenuToggleButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText)
	{
		return AddMenuToggleButton(RowText, InOutCurrentState, {.ToolTipText = ToolTipText});
	}

	bool AddMenuToggleButton(const FStringView& RowText, bool& InOutCurrentState, const FMenuButtonParams& Params)
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot add menu items without a current active menu anchor"));

		return Anchor->AddMenuToggleButton(RowText, InOutCurrentState, Params.ToolTipText);
	}

	bool AddMenuCheckButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText)
	{
		return AddMenuCheckButton(RowText, InOutCurrentState, {.ToolTipText = ToolTipText});
	}

	bool AddMenuCheckButton(const FStringView& RowText, bool& InOutCurrentState, const FMenuButtonParams& Params)
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot add menu items without a current active menu anchor"));

		return Anchor->AddMenuCheckButton(RowText, InOutCurrentState, Params.ToolTipText);
	}

	void BeginSubMenu(const FStringView& SubMenuText)
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot begin a submenu without a current active menu anchor"));

		Anchor->BeginSubMenu(SubMenuText);
	}

	void EndSubMenu()
	{
		TSharedPtr<SImContextMenuAnchor> Anchor = FSlateIMManager::Get().GetCurrentMenuRoot();
		checkf(Anchor, TEXT("Cannot end a submenu without a current active menu anchor"));

		Anchor->EndSubMenu();
	}
}
