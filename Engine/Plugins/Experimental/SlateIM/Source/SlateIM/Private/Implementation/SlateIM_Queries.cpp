// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SlateIMManager.h"
#include "SlateIM.h"

namespace SlateIM
{
	bool IsHovered()
	{
		if (TSharedPtr<SWidget> ChildWidget = FSlateIMManager::Get().GetCurrentChildAsWidget())
		{
			return ChildWidget->IsHovered();
		}

		return false;
	}

	bool IsFocused(EFocusDepth Depth)
	{
		if (TSharedPtr<SWidget> ChildWidget = FSlateIMManager::Get().GetCurrentChildAsWidget())
		{
			switch (Depth)
			{
			case EFocusDepth::SelfOnly:
				return ChildWidget->HasAnyUserFocus().IsSet();
			case EFocusDepth::IncludingDescendants:
				return ChildWidget->HasAnyUserFocusOrFocusedDescendants();
			default:
				checkNoEntry();
				break;
			}
		}

		return false;
	}
}
