// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Containers/Set.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/SlateIMManager.h"
#include "Roots/SlateIMExposedRoot.h"
#include "Roots/SlateIMViewportRoot.h"
#include "Roots/SlateIMWindowRoot.h"

#if WITH_ENGINE
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#endif

namespace SlateIM
{
	bool BeginWindowRoot(FName UniqueName, const FStringView& WindowTitle, FVector2f WindowSize, bool bShouldReopen)
	{
		return BeginWindowRoot(
			UniqueName,
			{
				.WindowTitle   = WindowTitle,
				.WindowSize    = WindowSize,
				.bShouldReopen = bShouldReopen
			}
		);
	}
	
	bool BeginWindowRoot(FName UniqueName, const FWindowParams& Params)
	{
		if (!FSlateIMManager::Get().CanUpdateSlateIM())
		{
			return false;
		}

		bool bNeedsCreate = true;

		FRootNode* RootNode = FSlateIMManager::Get().FindRoot<FSlateIMWindowRoot>(UniqueName);
		if (RootNode)
		{
			// Window existed and was destroyed and needs to be reopened if requested
			bNeedsCreate = !RootNode->RootWidget.IsValid() && Params.bShouldReopen;

			RootNode->RootState = RootNode->RootWidget->IsVisible();
		}

		TSharedPtr<FSlateIMWindowRoot> WindowRoot;
		if (bNeedsCreate)
		{
			// TODO - Save and restore window size and position, mapped to UniqueName
			TSharedRef<SWindow> NewWindowWidget
				= SNew(SWindow)
				.Title(FText::FromStringView(Params.WindowTitle))
				.ClientSize(Params.WindowSize)
				.IsTopmostWindow(Params.bAlwaysOnTop);

			FSlateApplication::Get().AddWindow(NewWindowWidget);

			WindowRoot = MakeShared<FSlateIMWindowRoot>(NewWindowWidget);

			if (RootNode)
			{
				RootNode->RootWidget = WindowRoot;
				RootNode->RootState = true;
			}
			else
			{
				RootNode = &FSlateIMManager::Get().AddRoot(UniqueName, WindowRoot);
			}
		}
		else
		{
			WindowRoot = StaticCastSharedPtr<FSlateIMWindowRoot>(RootNode->RootWidget);
		}
		
		WindowRoot->UpdateWindow(Params.WindowTitle);

		FSlateIMManager::Get().BeginRoot(UniqueName);

		return RootNode->RootState;
	}

#if WITH_ENGINE
	bool BeginViewportRoot(FName UniqueName, UGameViewportClient* ViewportClient, const FViewportRootLayout& Layout)
	{
		return BeginViewportRoot(UniqueName, ViewportClient, {.Layout = Layout});
	}

	bool BeginViewportRoot(FName UniqueName, UGameViewportClient* ViewportClient, const FViewportParams& Params)
	{
		if (!FSlateIMManager::Get().CanUpdateSlateIM())
		{
			return false;
		}
		
		bool bNeedsCreate = true;

		FRootNode* RootNode = FSlateIMManager::Get().FindRoot<FSlateIMViewportRoot>(UniqueName);
		if (RootNode)
		{
			const bool bIsValidRoot = RootNode->RootWidget.IsValid() && StaticCastSharedPtr<FSlateIMViewportRoot>(RootNode->RootWidget)->GameViewport == ViewportClient;
			bNeedsCreate = !bIsValidRoot;

			RootNode->RootState = bIsValidRoot;
		}

		TSharedPtr<FSlateIMViewportRoot> ViewportRoot;
		if (bNeedsCreate)
		{
			ViewportRoot = MakeShared<FSlateIMViewportRoot>(ViewportClient);

			RootNode = &FSlateIMManager::Get().AddRoot(UniqueName, ViewportRoot);
		}
		else
		{
			ViewportRoot = StaticCastSharedPtr<FSlateIMViewportRoot>(RootNode->RootWidget);
		}

		ViewportRoot->UpdateViewport(Params.Layout);

		FSlateIMManager::Get().BeginRoot(UniqueName);

		return RootNode->RootState;
	}

	bool BeginViewportRoot(FName UniqueName, ULocalPlayer* LocalPlayer, const FViewportRootLayout& Layout)
	{
		return BeginViewportRoot(UniqueName, LocalPlayer, {.Layout = Layout});
	}

	bool BeginViewportRoot(FName UniqueName, ULocalPlayer* LocalPlayer, const FViewportParams& Params)
	{
		if (!FSlateIMManager::Get().CanUpdateSlateIM())
		{
			return false;
		}
		
		bool bNeedsCreate = true;

		FRootNode* RootNode = FSlateIMManager::Get().FindRoot<FSlateIMViewportRoot>(UniqueName);
		if (RootNode)
		{
			const bool bIsValidRoot = RootNode->RootWidget.IsValid() && StaticCastSharedPtr<FSlateIMViewportRoot>(RootNode->RootWidget)->LocalPlayer == LocalPlayer;
			bNeedsCreate = !bIsValidRoot;

			RootNode->RootState = bIsValidRoot;
		}

		TSharedPtr<FSlateIMViewportRoot> ViewportRoot;
		if (bNeedsCreate)
		{
			ViewportRoot = MakeShared<FSlateIMViewportRoot>(LocalPlayer);

			RootNode = &FSlateIMManager::Get().AddRoot(UniqueName, ViewportRoot);
		}
		else
		{
			ViewportRoot = StaticCastSharedPtr<FSlateIMViewportRoot>(RootNode->RootWidget);
		}

		ViewportRoot->UpdateViewport(Params.Layout);

		FSlateIMManager::Get().BeginRoot(UniqueName);

		return RootNode->RootState;
	}

#if WITH_EDITOR
	bool BeginViewportRoot(FName UniqueName, TSharedPtr<IAssetViewport> AssetViewport, const FViewportRootLayout& Layout)
	{
		return BeginViewportRoot(UniqueName, AssetViewport, {.Layout = Layout});
	}

	bool BeginViewportRoot(FName UniqueName, TSharedPtr<IAssetViewport> AssetViewport, const FViewportParams& Params)
	{
		if (!FSlateIMManager::Get().CanUpdateSlateIM())
		{
			return false;
		}
		
		bool bNeedsCreate = true;

		FRootNode* RootNode = FSlateIMManager::Get().FindRoot<FSlateIMViewportRoot>(UniqueName);
		if (RootNode)
		{
			const bool bIsValidRoot = RootNode->RootWidget.IsValid() && StaticCastSharedPtr<FSlateIMViewportRoot>(RootNode->RootWidget)->AssetViewport == AssetViewport;
			bNeedsCreate = !bIsValidRoot;

			RootNode->RootState = bIsValidRoot;
		}
		
		TSharedPtr<FSlateIMViewportRoot> ViewportRoot;
		if (bNeedsCreate)
		{
			ViewportRoot = MakeShared<FSlateIMViewportRoot>(AssetViewport);

			RootNode = &FSlateIMManager::Get().AddRoot(UniqueName, ViewportRoot);
		}
		else
		{
			ViewportRoot = StaticCastSharedPtr<FSlateIMViewportRoot>(RootNode->RootWidget);
		}

		ViewportRoot->UpdateViewport(Params.Layout);

		FSlateIMManager::Get().BeginRoot(UniqueName);

		return RootNode->RootState;
	}
#endif // WITH_EDITOR

#endif // WITH_ENGINE

	bool BeginExposedRoot(FName UniqueName, TSharedPtr<SWidget>& OutSlateIMWidget)
	{
		if (!FSlateIMManager::Get().CanUpdateSlateIM())
		{
			return false;
		}
		
		bool bNeedsCreate = true;
		
		FRootNode* ExistingRoot = FSlateIMManager::Get().FindRoot<FSlateIMExposedRoot>(UniqueName);
		if (ExistingRoot)
		{
			ExistingRoot->RootState = ExistingRoot->RootWidget.IsValid();
			bNeedsCreate = !ExistingRoot->RootWidget.IsValid();
		}
		
		if (bNeedsCreate)
		{
			TSharedRef<FSlateIMExposedRoot> NewRoot = MakeShared<FSlateIMExposedRoot>();
			ExistingRoot = &FSlateIMManager::Get().AddRoot(UniqueName, NewRoot);
			OutSlateIMWidget = NewRoot->GetExposedWidget();
		}
		else if (TSharedPtr<FSlateIMExposedRoot> ExposedRoot = StaticCastSharedPtr<FSlateIMExposedRoot>(ExistingRoot->RootWidget))
		{
			OutSlateIMWidget = ExposedRoot->GetExposedWidget();
		}

		FSlateIMManager::Get().BeginRoot(UniqueName);
		
		return ExistingRoot->RootState;
	}

	void EndRoot()
	{
		FSlateIMManager::Get().EndRoot();
	}
}
