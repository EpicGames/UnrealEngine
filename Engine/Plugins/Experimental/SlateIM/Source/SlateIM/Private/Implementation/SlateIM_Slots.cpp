// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Misc/SlateIMManager.h"

namespace SlateIM
{
	void Padding(const FMargin NextPadding)
	{
		FSlateIMManager::Get().NextPadding = NextPadding;
	}

	void HAlign(EHorizontalAlignment NextAlignment)
	{
		FSlateIMManager::Get().NextHAlign = NextAlignment;
	}
	
	void VAlign(EVerticalAlignment NextAlignment)
	{
		FSlateIMManager::Get().NextVAlign = NextAlignment;
	}

	void AutoSize()
	{
		FSlateIMManager::Get().NextAutoSize = true;
	}

	void Fill()
	{
		FSlateIMManager::Get().NextAutoSize = false;
	}

	void MinWidth(float InMinWidth)
	{
		FSlateIMManager::Get().NextMinWidth = InMinWidth;
	}

	void MinHeight(float InMinHeight)
	{
		FSlateIMManager::Get().NextMinHeight = InMinHeight;
	}

	void MaxWidth(float InMaxWidth)
	{
		FSlateIMManager::Get().NextMaxWidth = InMaxWidth;
	}

	void MaxHeight(float InMaxHeight)
	{
		FSlateIMManager::Get().NextMaxHeight = InMaxHeight;
	}

	void Maximize()
	{
		HAlign(HAlign_Fill);
		VAlign(VAlign_Fill);
		Fill();
	}
}
