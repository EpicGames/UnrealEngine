// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Containers/SImTab.h"
#include "Containers/SImTabGroup.h"
#include "Misc/SlateIMLogging.h"
#include "Misc/SlateIMManager.h"
#include "Misc/SlateIMWidgetScope.h"

namespace SlateIM
{	
	void BeginTabGroup(const FName& TabGroupId)
	{
		TSharedPtr<SImTabGroup> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImTabGroup> Scope;
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget || ContainerWidget->GetTabGroupId() != TabGroupId)
			{
				ContainerWidget = SNew(SImTabGroup)
					.TabGroupId(TabGroupId);
				Scope.UpdateWidget(ContainerWidget);
			}

			ContainerWidget->BeginTabGroupUpdates();
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));
	}
	
	void EndTabGroup()
	{
		if (TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>())
		{
			TabGroup->FinalizeTabGroup();
		}
		FSlateIMManager::Get().PopContainer<SImTabGroup>();
	}

	void BeginTabStack()
	{
		TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>();
		if (ensureMsgf(TabGroup, TEXT("Cannot begin a Tab Stack - the current container is not a Tab Group. Are your Begin and End function calls mismatched?")))
		{
			TabGroup->BeginTabStack();
		}
	}

	void EndTabStack()
	{
		TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>();
		if (ensureMsgf(TabGroup, TEXT("Cannot end a Tab Stack - the current container is not a Tab Group. Are your Begin and End function calls mismatched?")))
		{
			TabGroup->EndTabLayoutNode();
		}
	}

	void BeginTabSplitter(EOrientation Orientation)
	{
		TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>();
		if (ensureMsgf(TabGroup, TEXT("Cannot begin a Tab Splitter - the current container is not a Tab Group. Are your Begin and End function calls mismatched?")))
		{
			TabGroup->BeginTabSplitter(Orientation);
		}
	}

	void EndTabSplitter()
	{
		TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>();
		if (ensureMsgf(TabGroup, TEXT("Cannot end a Tab Splitter - the current container is not a Tab Group. Are your Begin and End function calls mismatched?")))
		{
			TabGroup->EndTabLayoutNode();
		}
	}

	void TabSplitterSizeCoefficient(float SizeCoefficient)
	{
		TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>();
		if (ensureMsgf(TabGroup, TEXT("Cannot set the Tab Splitter Size Coefficient - the current container is not a Tab Group. Are your Begin and End function calls mismatched?")))
		{
			TabGroup->SetSizeCoefficient(SizeCoefficient);
		}
	}

	bool BeginTab(const FName& TabId, const FSlateIcon& TabIcon, const FText& TabTitle)
	{
		return BeginTab(TabId, {.TabIcon = TabIcon, .TabTitle = TabTitle});
	}

	bool BeginTab(const FName& TabId, const FTabParams& Params)
	{
		TSharedPtr<SImTabGroup> TabGroup = FSlateIMManager::Get().GetCurrentContainer<SImTabGroup>();
		if (!ensureMsgf(TabGroup, TEXT("Cannot begin a Tab - the current container is not a Tab Group. Are your Begin and End function calls mismatched?")))
		{
			return false;
		}
		
		TSharedPtr<SImTab> ContainerWidget;

		// Note widget scope must fall out of scope before we push container or else we are pushing this new container into itself instead of into its parent
		{
			FWidgetScope<SImTab> Scope;
			ContainerWidget = Scope.GetWidget();

			if (!ContainerWidget || ContainerWidget->TabId != TabId)
			{
				ContainerWidget = SNew(SImTab)
					.CanEverClose(false);
				ContainerWidget->TabId = TabId;
				ContainerWidget->SetOnTabDrawerOpened(FSimpleDelegate::CreateSP(ContainerWidget.Get(), &SImTab::OnDrawerOpened));
				ContainerWidget->SetOnTabDrawerClosed(FSimpleDelegate::CreateSP(ContainerWidget.Get(), &SImTab::OnDrawerClosed));
				Scope.UpdateWidget(ContainerWidget);

				// Params.TabIcon and Params.TabTitle are not updated after construction, this could be supported in the future but is not currently part of the Tab Manager API
				ContainerWidget->TabIcon = Params.TabIcon;
				ContainerWidget->TabTitle = Params.TabTitle;
			}

			if (!FSlateIMManager::Get().GetMutableCurrentRoot().Tabs.Contains(TabId))
			{
				FSlateIMManager::Get().GetMutableCurrentRoot().Tabs.Add(TabId, ContainerWidget);
			}
		}

		FSlateIMManager::Get().PushContainer(FContainerNode(ContainerWidget));

		return TabGroup->IsTabForegrounded(TabId);
	}

	void EndTab()
	{
		FSlateIMManager::Get().PopContainer<SImTab>();
	}
	
	void ActivateTab(const FName& TabId)
	{
		if (ensureMsgf(FSlateIMManager::Get().HasCurrentRoot(), TEXT("ActivateTab called without a current root. Have you started a root?!")))
		{
			FSlateIMManager::Get().GetMutableCurrentRoot().TabsToActivate.Add(TabId);
		}
	}
}
