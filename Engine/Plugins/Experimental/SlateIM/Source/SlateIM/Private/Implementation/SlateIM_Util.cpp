// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MessageDialog.h"
#include "SlateIM.h"

#include "Misc/SlateIMManager.h"

namespace SlateIM
{
	bool CanUpdateSlateIM()
	{
		return FSlateIMManager::Get().CanUpdateSlateIM();
	}
	
	void BeginDisabledState()
	{
		FSlateIMManager::Get().GetMutableCurrentRoot().SetDisabledState();
	}

	void EndDisabledState()
	{
		FSlateIMManager::Get().GetMutableCurrentRoot().SetEnabledState();
	}

	void SetToolTip(const FStringView& NextToolTip)
	{
		FSlateIMManager::Get().GetMutableCurrentRoot().SetNextToolTip(NextToolTip);
	}

	EAppReturnType::Type ModalDialog(EAppMsgType::Type MessageType, const FStringView& DialogText, EAppMsgCategory Category, const FStringView& DialogTitle)
	{
		return ModalDialog(MessageType, DialogText, {.Category = Category, .Title = DialogTitle});
	}

	EAppReturnType::Type ModalDialog(EAppMsgType::Type MessageType, const FStringView& DialogText, const FModalDialogParams& Params)
	{
		FSlateIMManager::Get().OnSlateIMModalOpened();

		ON_SCOPE_EXIT
		{
			FSlateIMManager::Get().OnSlateIMModalClosed();
		};
		return FMessageDialog::Open(Params.Category, MessageType, FText::FromStringView(DialogText), FText::FromStringView(Params.Title));
	}
}
