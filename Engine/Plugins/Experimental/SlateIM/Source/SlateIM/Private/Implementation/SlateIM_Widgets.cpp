// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"
#include "SlateIM_Internal.h"

#include "Misc/SlateIMLogging.h"
#include "Misc/SlateIMManager.h"
#include "Misc/SlateIMSlateResources.h"
#include "Misc/SlateIMSlotData.h"
#include "Misc/SlateIMWidgetScope.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SImButton.h"
#include "Widgets/SImCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_ENGINE
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/SlateIMSlateResources.h"
#include "SlateMaterialBrush.h"
#endif

namespace SlateIM
{
	void Text(const FStringView& InText, const FTextBlockStyle* TextStyle /*= nullptr*/)
	{
		Text(InText, FSlateColor::UseForeground(), TextStyle);
	}

	void Text(const FStringView& InText, FSlateColor Color, const FTextBlockStyle* TextStyle /*= nullptr*/)
	{
		Text(InText, {.Color = Color, .Style = TextStyle});
	}

	void Text(const FStringView& InText, const FTextParams& Params)
	{
		FWidgetScope<STextBlock> Scope;
		TSharedPtr<STextBlock> TextBlock = Scope.GetOrCreate([&InText, Params] ()
			{
				TSharedPtr<STextBlock> TextBlock =
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(FText::FromStringView(InText))
					.ColorAndOpacity(Params.Color);

				if (Params.Style)
				{
					TextBlock->SetTextStyle(Params.Style);
				}
				return TextBlock;
			});

		Scope.HashData(Params.Color);
		Scope.HashData(Params.Style);
		Scope.HashStringView(InText);

		if (Scope.IsDataHashDirty())
		{
			TextBlock->SetText(FText::FromStringView(InText));
			TextBlock->SetTextStyle(Params.Style);
			TextBlock->SetColorAndOpacity(Params.Color);
		}
	}

	bool EditableText(FString& InOutText, const FStringView& HintText, const FEditableTextBoxStyle* TextStyle)
	{
		return EditableText(InOutText, {.HintText = HintText, .Style = TextStyle});
	}

	bool EditableText(FString& InOutText, const FEditableTextParams& Params)
	{
		FWidgetScope<SEditableTextBox> Scope;
		TSharedPtr<SEditableTextBox> EditableText = Scope.GetWidget();
		const float MinWidth = FSlateIMManager::Get().NextMinWidth.Get(Defaults::InputWidgetWidth);

		bool bWasActivated = false;

		Scope.HashData(Params.Style);
		Scope.HashStringView(InOutText);
		Scope.HashStringView(Params.HintText);

		if (!EditableText)
		{
			TWeakPtr<FSlateIMWidgetActivationMetadata> ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr();

			EditableText =
				SNew(SEditableTextBox)
				.MinDesiredWidth(MinWidth)
				.Text(FText::FromStringView(InOutText))
				.HintText(FText::FromStringView(Params.HintText))
				.OnTextChanged_Lambda([ActivationData](const FText& NewText)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				})
				.OnTextCommitted_Lambda([ActivationData](const FText& NewText, ETextCommit::Type)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				});

			if (Params.Style)
			{
				EditableText->SetStyle(Params.Style);
			}
			
			Scope.UpdateWidget(EditableText);
		}
		else
		{
			bWasActivated = Scope.IsActivatedThisFrame();

			const bool bIsHashDirty = Scope.IsDataHashDirty();

			if (bIsHashDirty)
			{
				EditableText->SetStyle(Params.Style);
				EditableText->SetHintText(FText::FromStringView(Params.HintText));
			}
			
			if (bWasActivated)
			{
				InOutText = EditableText->GetText().ToString();
			}
			else if (bIsHashDirty)
			{
				EditableText->SetText(FText::FromStringView(InOutText));
			}
			
			EditableText->SetMinimumDesiredWidth(MinWidth);
		}

		return bWasActivated;
	}

	void Image_Internal(const FImageParams& Params, TFunctionRef<const FSlateBrush*(const TSharedRef<SImage>&)> GetBrush)
	{
		FWidgetScope<SImage> Scope(Defaults::Padding, HAlign_Left, VAlign_Top);
		TSharedPtr<SImage> ImageWidget = Scope.GetWidget();

		if (!ImageWidget)
		{
			ImageWidget =
				SNew(SImage)
				.ColorAndOpacity(Params.ColorAndOpacity)
				.DesiredSizeOverride(Params.DesiredSize == FVector2D::ZeroVector ? TOptional<FVector2D>() : Params.DesiredSize);

			ImageWidget->SetImage(GetBrush(ImageWidget.ToSharedRef()));
			Scope.UpdateWidget(ImageWidget);
		}
		else
		{
			ImageWidget->SetImage(GetBrush(ImageWidget.ToSharedRef()));
			ImageWidget->SetColorAndOpacity(Params.ColorAndOpacity);
			ImageWidget->SetDesiredSizeOverride(Params.DesiredSize == FVector2D::ZeroVector ? TOptional<FVector2D>() : Params.DesiredSize);
		}
	}

	void Image(const FSlateBrush* ImageBrush, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize)
	{
		Image_Internal({.ColorAndOpacity = ColorAndOpacity, .DesiredSize = DesiredSize}, [ImageBrush](const TSharedRef<SImage>&) { return ImageBrush; });
	}

	void Image(const FSlateBrush* ImageBrush, const FImageParams& Params)
	{
		Image_Internal(Params, [ImageBrush](const TSharedRef<SImage>&) { return ImageBrush; });
	}

	void Image(const FName ImageStyleName, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize)
	{
		Image(FAppStyle::Get().GetBrush(ImageStyleName), {.ColorAndOpacity = ColorAndOpacity, .DesiredSize = DesiredSize});
	}

	void Image(const FName ImageStyleName, const FImageParams& Params)
	{
		Image(FAppStyle::Get().GetBrush(ImageStyleName), Params);
	}

	void Image(const FSlateColor& ColorAndOpacity, FVector2D DesiredSize)
	{
		Image({.ColorAndOpacity = ColorAndOpacity, .DesiredSize = DesiredSize});
	}

	void Image(const FImageParams& Params)
	{
		static FSlateBrush DefaultBrush;
		Image(&DefaultBrush, Params);
	}

#if WITH_ENGINE
	void Image(UTexture2D* ImageTexture, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize)
	{
		Image(ImageTexture, {.ColorAndOpacity = ColorAndOpacity, .DesiredSize = DesiredSize});
	}

	void Image(UTexture2D* ImageTexture, const FImageParams& Params)
	{
		auto MakeTextureBrush = [ImageTexture]() -> FSlateBrush
		{
			if (ImageTexture)
			{
				FSlateBrush Brush;
				Brush.SetResourceObject(ImageTexture);
				Brush.ImageSize = FVector2D(ImageTexture->GetSizeX(), ImageTexture->GetSizeY());
				return Brush;
			}
			return FSlateNoResource();
		};
		
		Image_Internal(Params, [&MakeTextureBrush, ImageTexture](const TSharedRef<SImage>& ImageWidget)
		{
			TSharedPtr<FUObjectImageResource> Resource = ImageWidget->GetMetaData<FUObjectImageResource>();
			if (!Resource)
			{
				Resource = MakeShared<FUObjectImageResource>(FPinnedImageResource{ MakeTextureBrush(), TStrongObjectPtr<UObject>(ImageTexture) });
				ImageWidget->AddMetadata(Resource.ToSharedRef());
			}
			else if (Resource->Data.PinnedResource.Get() != ImageTexture)
			{
				Resource->Data = FPinnedImageResource{ MakeTextureBrush(), TStrongObjectPtr<UObject>(ImageTexture) };
			}

			return &Resource->Data.Brush;
		});
	}

	void Image(UTextureRenderTarget2D* ImageRenderTarget, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize)
	{
		Image(ImageRenderTarget, {.ColorAndOpacity = ColorAndOpacity, .DesiredSize = DesiredSize});
	}

	void Image(UTextureRenderTarget2D* ImageRenderTarget, const FImageParams& Params)
	{
		auto MakeRTBrush = [ImageRenderTarget]() -> FSlateBrush
		{
			if (ImageRenderTarget)
			{
				FSlateBrush Brush;
				Brush.SetResourceObject(ImageRenderTarget);
				Brush.ImageSize = FVector2D(ImageRenderTarget->SizeX, ImageRenderTarget->SizeY);
				return Brush;
			}
			return FSlateNoResource();
		};
		
		Image_Internal(Params, [&MakeRTBrush, ImageRenderTarget](const TSharedRef<SImage>& ImageWidget)
		{
			TSharedPtr<FUObjectImageResource> Resource = ImageWidget->GetMetaData<FUObjectImageResource>();
			if (!Resource)
			{
				Resource = MakeShared<FUObjectImageResource>(FPinnedImageResource{ MakeRTBrush(), TStrongObjectPtr<UObject>(ImageRenderTarget) });
				ImageWidget->AddMetadata(Resource.ToSharedRef());
			}
			else if (Resource->Data.PinnedResource.Get() != ImageRenderTarget)
			{
				Resource->Data = FPinnedImageResource{ MakeRTBrush(), TStrongObjectPtr<UObject>(ImageRenderTarget) };
			}

			return &Resource->Data.Brush;
		});
	}

	void Image(UMaterialInterface* ImageMaterial, FVector2D BrushSize, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize)
	{
		Image(ImageMaterial, BrushSize, {.ColorAndOpacity = ColorAndOpacity, .DesiredSize = DesiredSize});
	}

	void Image(UMaterialInterface* ImageMaterial, FVector2D BrushSize, const FImageParams& Params)
	{
		auto MakeMaterialBrush = [ImageMaterial, BrushSize]() -> FSlateBrush
		{
			if (ImageMaterial)
			{
				return FSlateMaterialBrush(*ImageMaterial, BrushSize);
			}
			return FSlateNoResource();
		};
		
		Image_Internal(Params, [&MakeMaterialBrush, ImageMaterial](const TSharedRef<SImage>& ImageWidget)
		{
			TSharedPtr<FUObjectImageResource> Resource = ImageWidget->GetMetaData<FUObjectImageResource>();
			if (!Resource)
			{
				Resource = MakeShared<FUObjectImageResource>(FPinnedImageResource{ MakeMaterialBrush(), TStrongObjectPtr<UObject>(ImageMaterial) });
				ImageWidget->AddMetadata(Resource.ToSharedRef());
			}
			else if (Resource->Data.PinnedResource.Get() != ImageMaterial)
			{
				Resource->Data = FPinnedImageResource{ MakeMaterialBrush(), TStrongObjectPtr<UObject>(ImageMaterial) };
			}

			return &Resource->Data.Brush;
		});
	}
#endif

	bool Button(const FStringView& InText, const FButtonStyle* InStyle)
	{
		return Button(InText, {.Style = InStyle});
	}

	bool Button(const FStringView& InText, const bool bEnabled, const FButtonStyle* InStyle)
	{
		return Button(InText, {.bEnabled = bEnabled, .Style = InStyle});
	}

	bool Button(const FStringView& InText, const FButtonParams& Params)
	{
		if (!Params.bEnabled)
		{
			BeginDisabledState();
		}

		FWidgetScope<SImButton> Scope;
		TSharedPtr<SImButton> ButtonWidget = Scope.GetWidget();

		Scope.HashStringView(InText);
		Scope.HashData(Params.Style);

		bool bWasClicked = false;
		if (!ButtonWidget)
		{
			ButtonWidget =
				SNew(SImButton)
				.OnClicked_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()]()
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
					return FReply::Handled();
				});

			ButtonWidget->SetText(InText);

			if (Params.Style)
			{
				ButtonWidget->SetButtonStyle(Params.Style);
			}

			Scope.UpdateWidget(ButtonWidget);
		}
		else
		{
			bWasClicked = Scope.IsActivatedThisFrame();

			if (Scope.IsDataHashDirty())
			{
				ButtonWidget->SetText(InText);
				if (Params.Style)
				{
					ButtonWidget->SetButtonStyle(Params.Style);
				}
			}
		}

		if (!Params.bEnabled)
		{
			EndDisabledState();
		}

		return bWasClicked;
	}


	bool CheckBox(const FStringView& InLabel, bool& InOutCurrentState, const FCheckBoxStyle* CheckBoxStyle)
	{
		return CheckBox(InOutCurrentState, {.Label = InLabel, .CheckBoxStyle = CheckBoxStyle});
	}

	bool CheckBox(bool& InOutCurrentState, const FCheckBoxParams& Params)
	{
		ECheckBoxState CurrentEnumState = InOutCurrentState ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

		const bool bValueChanged = CheckBox(CurrentEnumState, Params);
		if (bValueChanged)
		{
			InOutCurrentState = CurrentEnumState == ECheckBoxState::Checked;
		}

		return bValueChanged;
	}

	bool CheckBox(const FStringView& InLabel, ECheckBoxState& InOutCurrentState, const FCheckBoxStyle* CheckBoxStyle)
	{
		return CheckBox(InOutCurrentState, {.Label = InLabel, .CheckBoxStyle = CheckBoxStyle});
	}

	bool CheckBox(ECheckBoxState& InOutCurrentState, const FCheckBoxParams& Params)
	{
		FWidgetScope<SImCheckBox> Scope;
		TSharedPtr<SImCheckBox> CheckboxWidget = Scope.GetWidget();

		bool bValueChanged = false;

		Scope.HashStringView(Params.Label);
		Scope.HashData(Params.CheckBoxStyle);

		if (!CheckboxWidget)
		{
			CheckboxWidget =
				SNew(SImCheckBox)
				.IsChecked(InOutCurrentState)
				.OnCheckStateChanged_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()](ECheckBoxState NewState)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				});

			CheckboxWidget->SetText(FText::FromStringView(Params.Label));

			if (Params.CheckBoxStyle)
			{
				CheckboxWidget->SetStyle(Params.CheckBoxStyle);
			}

			Scope.UpdateWidget(CheckboxWidget);
		}
		else
		{
			bValueChanged = Scope.IsActivatedThisFrame();
			if (bValueChanged)
			{
				InOutCurrentState = CheckboxWidget->GetCheckedState();
			}
			else
			{
				CheckboxWidget->SetIsChecked(InOutCurrentState);
			}

			if (Scope.IsDataHashDirty())
			{
				CheckboxWidget->SetText(FText::FromStringView(Params.Label));
				if (Params.CheckBoxStyle)
				{
					CheckboxWidget->SetStyle(Params.CheckBoxStyle);
				}
			}
		}

		return bValueChanged;
	}

	template<typename NumericType>
	bool SpinBox_Internal(NumericType& InOutValue, const FSpinBoxParams<NumericType>& Params)
	{
		FWidgetScope<SSpinBox<NumericType>> Scope(Defaults::Padding, Defaults::HAlign, Defaults::VAlign, Defaults::bAutoSize, Defaults::InputWidgetWidth);
		TSharedPtr<SSpinBox<NumericType>> SpinBoxWidget = Scope.GetWidget();

		Scope.HashData(Params.Style);

		class FSpinBoxState : public ISlateMetaData
		{
		public:
			SLATE_METADATA_TYPE(FSpinBoxState, ISlateMetaData)
			bool bIsChanging = false;
		};

		bool bValueChanged = false;
		if (!SpinBoxWidget)
		{
			TSharedRef<FSpinBoxState> SpinBoxState = MakeShared<FSpinBoxState>();
			TWeakPtr<FSlateIMWidgetActivationMetadata> ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr();
			SpinBoxWidget =
				SNew(SSpinBox<NumericType>)
				.MinValue(Params.Min)
				.MaxValue(Params.Max)
				.Value(InOutValue)
				.OnBeginSliderMovement_Lambda([SpinBoxState](){ SpinBoxState->bIsChanging = true; })
				.OnEndSliderMovement_Lambda([SpinBoxState, ActivationData](NumericType NewVal)
				{
					SpinBoxState->bIsChanging = false;
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				})
				.OnValueChanged_Lambda([ActivationData](NumericType NewVal)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				})
				.OnValueCommitted_Lambda([ActivationData](NumericType NewVal, ETextCommit::Type CommitType)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				});

			SpinBoxWidget->AddMetadata(SpinBoxState);
			Scope.UpdateWidget(SpinBoxWidget);

			if (Params.Style)
			{
				SpinBoxWidget->SetWidgetStyle(Params.Style);
			}
		}
		else
		{
			SpinBoxWidget->SetMinValue(Params.Min);
			SpinBoxWidget->SetMaxValue(Params.Max);

			bValueChanged = Scope.IsActivatedThisFrame();
			if (bValueChanged)
			{
				InOutValue = SpinBoxWidget->GetValue();
			}
			else if (SpinBoxWidget->GetValue() != InOutValue)
			{
				TSharedPtr<FSpinBoxState> SpinBoxState = SpinBoxWidget->template GetMetaData<FSpinBoxState>();
				if (ensure(SpinBoxState) && !SpinBoxState->bIsChanging)
				{
					SpinBoxWidget->SetValue(InOutValue);
				}
			}

			if (Params.Style && Scope.IsDataHashDirty())
			{
				SpinBoxWidget->SetWidgetStyle(Params.Style);
				SpinBoxWidget->InvalidateStyle();
			}
		}

		return bValueChanged;
	}


	bool SpinBox(float& InOutValue, TOptional<float> Min, TOptional<float> Max, const FSpinBoxStyle* SpinBoxStyle)
	{
		return SpinBox_Internal<float>(InOutValue, {.Min = Min, .Max = Max, .Style = SpinBoxStyle});
	}

	bool SpinBox(double& InOutValue, TOptional<double> Min, TOptional<double> Max, const FSpinBoxStyle* SpinBoxStyle)
	{
		return SpinBox_Internal<double>(InOutValue, {.Min = Min, .Max = Max, .Style = SpinBoxStyle});
	}

	bool SpinBox(int32& InOutValue, TOptional<int32> Min, TOptional<int32> Max, const FSpinBoxStyle* SpinBoxStyle)
	{
		return SpinBox_Internal<int32>(InOutValue, {.Min = Min, .Max = Max, .Style = SpinBoxStyle});
	}

	template<>
	bool SpinBox<float>(float& InOutValue, const FSpinBoxParams<float>& Params)
	{
		return SpinBox_Internal<float>(InOutValue, Params);
	}

	template<>
	bool SpinBox<double>(double& InOutValue, const FSpinBoxParams<double>& Params)
	{
		return SpinBox_Internal<double>(InOutValue, Params);
	}

	template<>
	bool SpinBox<int32>(int32& InOutValue, const FSpinBoxParams<int32>& Params)
	{
		return SpinBox_Internal<int32>(InOutValue, Params);
	}

	bool Slider(float& InOutValue, float Min, float Max, float Step, const FSliderStyle* SliderStyle /*= nullptr*/)
	{
		return Slider(InOutValue, {.Min = Min, .Max = Max, .Step = Step, .Style = SliderStyle});
	}

	bool Slider(float& InOutValue, const FSliderParams& Params)
	{
		FWidgetScope<SSlider> Scope(Defaults::Padding, Defaults::HAlign, Defaults::VAlign, Defaults::bAutoSize, Defaults::InputWidgetWidth);
		TSharedPtr<SSlider> SliderWidget = Scope.GetWidget();

		Scope.HashData(Params.Style);

		bool bValueChanged = false;
		if (!SliderWidget)
		{
			SliderWidget =
				SNew(SSlider)
				.MinValue(Params.Min)
				.MaxValue(Params.Max)
				.StepSize(Params.Step)
				.Value(InOutValue)
				.OnValueChanged_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()](float NewVal)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				});

			if (Params.Style)
			{
				SliderWidget->SetStyle(Params.Style);
			}

			Scope.UpdateWidget(SliderWidget);
		}
		else
		{
			SliderWidget->SetMinAndMaxValues(Params.Min, Params.Max);
			SliderWidget->SetStepSize(Params.Step);

			bValueChanged = Scope.IsActivatedThisFrame();
			if (bValueChanged)
			{
				InOutValue = SliderWidget->GetValue();
			}
			else
			{
				SliderWidget->SetValue(InOutValue);
			}
			
			if (Scope.IsDataHashDirty() && Params.Style)
			{
				SliderWidget->SetStyle(Params.Style);
			}
		}

		return bValueChanged;
	}

	void ProgressBar(TOptional<float> Percent, const FProgressBarStyle* ProgressBarStyle /*= nullptr*/)
	{
		ProgressBar(Percent, {.Style = ProgressBarStyle});
	}

	void ProgressBar(TOptional<float> Percent, const FProgressBarParams& Params)
	{
		FWidgetScope<SProgressBar> Scope(Defaults::Padding, Defaults::HAlign, Defaults::VAlign, Defaults::bAutoSize, Defaults::InputWidgetWidth);
		TSharedPtr<SProgressBar> ProgressBarWidget = Scope.GetWidget();

		Scope.HashData(Params.Style);

		if (!ProgressBarWidget)
		{
			ProgressBarWidget =
				SNew(SProgressBar)
				.Percent(Percent);

			if (Params.Style)
			{
				ProgressBarWidget->SetStyle(Params.Style);
			}

			Scope.UpdateWidget(ProgressBarWidget);
		}
		else
		{
			ProgressBarWidget->SetPercent(Percent);
			
			if (Scope.IsDataHashDirty() && Params.Style)
			{
				ProgressBarWidget->SetStyle(Params.Style);
			}
		}
	}

	bool SearchableComboBox_Impl(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, const FComboBoxParams& Params)
	{
		FWidgetScope<SSearchableComboBox> Scope;
		TSharedPtr<SSearchableComboBox> ComboWidget = Scope.GetWidget();
		
		using FComboBoxData = FSlateIMDataStore<TArray<TSharedPtr<FString>>>;
		
		auto BuildComboOptions = [&]()
		{
			TArray<TSharedPtr<FString>> Options;
			Options.Reserve(ComboItems.Num());

			for (const FString& Item : ComboItems)
			{
				Options.Add(MakeShared<FString>(Item));
			}
			return Options;
		};

		bool bValueChanged = false;
		if (!ComboWidget)
		{

			TSharedRef<STextBlock> ComboText = SNew(STextBlock);
			TSharedRef<FComboBoxData> WidgetIMData = MakeShared<FComboBoxData>(BuildComboOptions());

			SAssignNew(ComboWidget, SSearchableComboBox)
			.OptionsSource(&WidgetIMData->Data)
			.InitiallySelectedItem(WidgetIMData->Data.IsValidIndex(InOutSelectedItemIndex) ? WidgetIMData->Data[InOutSelectedItemIndex] : nullptr)
			.ComboBoxStyle(Params.Style ? Params.Style : &FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox"))
			.Content()[ComboText]
			.OnSelectionChanged_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()](TSharedPtr<FString> NewVal, ESelectInfo::Type SelectType)
			{
				FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
			})
			.OnGenerateWidget_Lambda([] (TSharedPtr<FString> InItem)
			{
				return SNew(STextBlock)
					.Text(FText::FromString(*InItem));
			});

			const TAttribute<FText> ComboTextAttribute = TAttribute<FText>::CreateLambda(
				[WeakComboWidget = ComboWidget->AsWeak()]()
				{
					if (TSharedPtr<SSearchableComboBox> PinnedComboWidget = StaticCastSharedPtr<SSearchableComboBox>(WeakComboWidget.Pin()))
					{
						TSharedPtr<FString> SelectedText = PinnedComboWidget->GetSelectedItem();
						if (SelectedText.IsValid())
						{
							return FText::FromString(*SelectedText);
						}
					}

					return FText::FromString(TEXT("Select..."));
				}
			);

			ComboText->SetText(ComboTextAttribute);

			ComboWidget->AddMetadata(WidgetIMData);

			Scope.UpdateWidget(ComboWidget);
		}
		else
		{
			TSharedPtr<FComboBoxData> ComboBoxData = ComboWidget->GetMetaData<FComboBoxData>();
			if (Params.bForceRefresh)
			{
				ComboBoxData->Data = BuildComboOptions();
				if (InOutSelectedItemIndex != INDEX_NONE && !ComboWidget->IsOpen() && ComboBoxData->Data.IsValidIndex(InOutSelectedItemIndex))
				{
					ComboWidget->SetSelectedItem(ComboBoxData->Data[InOutSelectedItemIndex]);
				}
			}
			else
			{
				bValueChanged = Scope.IsActivatedThisFrame();
				if (bValueChanged)
				{
					TSharedPtr<FString> SelectedItem = ComboWidget->GetSelectedItem();
					const int32 NewSelectedIndex = SelectedItem.IsValid() ? ComboBoxData->Data.IndexOfByKey(SelectedItem) : INDEX_NONE;
					UE_LOGF(LogSlateIM, Verbose, "Combo Selection Changed %d -> %d", InOutSelectedItemIndex, NewSelectedIndex);
					InOutSelectedItemIndex = NewSelectedIndex;
				}
			}
		}

		return bValueChanged;
	}

	bool SearchableComboBox(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, bool bForceRefresh, const FComboBoxStyle* InComboStyle)
	{
		return SearchableComboBox_Impl(ComboItems, InOutSelectedItemIndex, {.bForceRefresh = bForceRefresh, .Style = InComboStyle});
	}

	bool ComboBox_Impl(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, const FComboBoxParams& Params)
	{
		FWidgetScope<STextComboBox> Scope;
		TSharedPtr<STextComboBox> ComboWidget = Scope.GetWidget();

		Scope.HashData(Params.Style);
		Scope.HashData(InOutSelectedItemIndex);
		
		using FComboBoxData = FSlateIMDataStore<TArray<TSharedPtr<FString>>>;
		
		auto BuildComboOptions = [&]()
		{
			TArray<TSharedPtr<FString>> Options;
			Options.Reserve(ComboItems.Num());

			for (const FString& Item : ComboItems)
			{
				Options.Add(MakeShared<FString>(Item));
			}
			return Options;
		};

		bool bValueChanged = false;
		if (!ComboWidget)
		{
			TSharedRef<FComboBoxData> WidgetIMData = MakeShared<FComboBoxData>(BuildComboOptions());

			ComboWidget =
				SNew(STextComboBox)
				.OptionsSource(&WidgetIMData->Data)
				.InitiallySelectedItem(WidgetIMData->Data.IsValidIndex(InOutSelectedItemIndex) ? WidgetIMData->Data[InOutSelectedItemIndex] : nullptr)
				.OnSelectionChanged_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()](TSharedPtr<FString> NewVal, ESelectInfo::Type SelectType)
				{
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				});

			if (Params.Style)
			{
				ComboWidget->SetStyle(Params.Style);
			}
			
			ComboWidget->AddMetadata(WidgetIMData);

			Scope.UpdateWidget(ComboWidget);
		}
		else
		{
			TSharedPtr<FComboBoxData> ComboBoxData = ComboWidget->GetMetaData<FComboBoxData>();
			if (Params.bForceRefresh)
			{
				ComboBoxData->Data = BuildComboOptions();
				if (InOutSelectedItemIndex != INDEX_NONE && !ComboWidget->IsOpen() && ComboBoxData->Data.IsValidIndex(InOutSelectedItemIndex))
				{
					ComboWidget->SetSelectedItem(ComboBoxData->Data[InOutSelectedItemIndex]);
				}
			}
			else
			{
				bValueChanged = Scope.IsActivatedThisFrame();
				if (bValueChanged)
				{
					TSharedPtr<FString> SelectedItem = ComboWidget->GetSelectedItem();
					const int32 NewSelectedIndex = SelectedItem.IsValid() ? ComboBoxData->Data.IndexOfByKey(SelectedItem) : INDEX_NONE;
					UE_LOGF(LogSlateIM, Verbose, "Combo Selection Changed %d -> %d", InOutSelectedItemIndex, NewSelectedIndex);
					InOutSelectedItemIndex = NewSelectedIndex;
				}
			}

			if (Scope.IsDataHashDirty())
			{
				TSharedPtr<FString> SelectedItem = ComboWidget->GetSelectedItem();
				const int32 SelectedIndex = SelectedItem.IsValid() ? ComboBoxData->Data.IndexOfByKey(SelectedItem) : INDEX_NONE;
				if (SelectedIndex != InOutSelectedItemIndex && ComboBoxData->Data.IsValidIndex(InOutSelectedItemIndex))
				{
					ComboWidget->SetSelectedItem(ComboBoxData->Data[InOutSelectedItemIndex]);
				}

				ComboWidget->SetStyle(Params.Style);
			}
		}

		return bValueChanged;
	}

	bool ComboBox(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, bool bForceRefresh, const FComboBoxStyle* InComboStyle)
	{
		return ComboBox_Impl(ComboItems, InOutSelectedItemIndex, {.bForceRefresh = bForceRefresh, .Style = InComboStyle});
	}

	bool ComboBox(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, const FComboBoxParams& Params)
	{
		if (Params.bSearchable)
		{
			return SearchableComboBox_Impl(ComboItems, InOutSelectedItemIndex, Params);
		}

		return ComboBox_Impl(ComboItems, InOutSelectedItemIndex, Params);
	}

	bool SelectionList(const TArray<FString>& ListItems, int32& InOutSelectedItemIndex, bool bForceRefresh, const FTableViewStyle* InStyle)
	{
		return SelectionList(ListItems, InOutSelectedItemIndex, {.bForceRefresh = bForceRefresh, .Style = InStyle});
	}

	bool SelectionList(const TArray<FString>& ListItems, int32& InOutSelectedItemIndex, const FSelectionListParams& Params)
	{
		SCOPED_NAMED_EVENT_TEXT("SlateIM::SelectionList", FColorList::Goldenrod);
		using ListViewType = SListView<TSharedPtr<FString>>;
		using FListViewData = FSlateIMDataStore<TArray<TSharedPtr<FString>>>;

		FWidgetScope<ListViewType> Scope;
		TSharedPtr<ListViewType> ListWidget = Scope.GetWidget();

		Scope.HashData(Params.Style);

		auto BuildListItems = [&]()
		{
			TArray<TSharedPtr<FString>> ListItemsSource;
			ListItemsSource.Reserve(ListItems.Num());
			for(const FString& Item : ListItems)
			{
				ListItemsSource.Add(MakeShared<FString>(Item));
			}
			return ListItemsSource;
		};

		bool bSelectionChanged = false;
		if (!ListWidget)
		{
			TSharedRef<FListViewData> ListViewData = MakeShared<FListViewData>(BuildListItems());
			ListWidget = SNew(ListViewType)
				.SelectionMode(ESelectionMode::Single)
				.ListViewStyle(Params.Style)
				.ListItemsSource(&(ListViewData->Data))
				.OnSelectionChanged_Lambda([ActivationData = Scope.GetOrCreateActivationMetadata().ToWeakPtr()](TSharedPtr<FString> NewVal, ESelectInfo::Type SelectType)
				{
					UE_LOGF(LogSlateIM, Verbose, "Selected %ls", NewVal.IsValid() ? **NewVal.Get() : TEXT("[NULL]"));
					FSlateIMManager::Get().ActivateWidget(ActivationData.Pin());
				})
				.OnGenerateRow_Lambda([](TSharedPtr<FString> ListItem, const TSharedRef<STableViewBase>& OwnerTable)
				{
					return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
						.Padding(4.f)
						[
							SNew(STextBlock).Text(FText::FromString(*ListItem))
						];
				});

			ListWidget->AddMetadata<FListViewData>(ListViewData);
			
			if (ListViewData->Data.IsValidIndex(InOutSelectedItemIndex))
			{
				ListWidget->SetItemSelection(ListViewData->Data[InOutSelectedItemIndex], true);
			}
			Scope.UpdateWidget(ListWidget);
		}
		else
		{
			TSharedPtr<FListViewData> ListViewData = ListWidget->GetMetaData<FListViewData>();
			if (Params.bForceRefresh)
			{
				ListViewData->Data = BuildListItems();
				ListWidget->RequestListRefresh();
			
				if (ListViewData->Data.IsValidIndex(InOutSelectedItemIndex))
				{
					ListWidget->SetItemSelection(ListViewData->Data[InOutSelectedItemIndex], true);
				}
				else
				{
					ListWidget->ClearSelection();
				}
			}
			else
			{
				bSelectionChanged = Scope.IsActivatedThisFrame();
				if (bSelectionChanged)
				{
					TArray<TSharedPtr<FString>> SelectedItems = ListWidget->GetSelectedItems();
					const int32 NewSelectedIndex = SelectedItems.Num() > 0 ? ListViewData->Data.IndexOfByKey(SelectedItems[0]) : INDEX_NONE;
					UE_LOGF(LogSlateIM, Verbose, "List Selection Changed %d -> %d", InOutSelectedItemIndex, NewSelectedIndex);
					InOutSelectedItemIndex = NewSelectedIndex;
				}
			}

			if (Scope.IsDataHashDirty())
			{
				ListWidget->SetStyle(Params.Style);
			}
		}

		return bSelectionChanged;
	}

	void Spacer(const FVector2D& Size)
	{
		FWidgetScope<SSpacer> Scope(Defaults::Padding, HAlign_Left, VAlign_Top);
		TSharedPtr<SSpacer> SpacerWidget = Scope.GetWidget();

		if (!SpacerWidget)
		{
			SpacerWidget = SNew(SSpacer)
				.Size(Size);
			Scope.UpdateWidget(SpacerWidget);
		}
		else
		{
			SpacerWidget->SetSize(Size);
		}
	}

	void Widget(TSharedRef<SWidget> InWidget)
	{
		FWidgetScope<SWidget> Scope(InWidget, Defaults::Padding, HAlign_Left, VAlign_Top);
		TSharedPtr<SWidget> Widget = Scope.GetWidget();

		if (!Widget)
		{
			Scope.UpdateWidget(InWidget);
		}
	}
}
