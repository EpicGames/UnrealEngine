// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Union.h"
#include "SlateIMTypeChecking.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

class SWidget;

// Interface for non-widget children in the SlateIM hierarchy (eg. for virtualized listview entries)
class ISlateIMChild : public ISlateIMTypeChecking
{
public:
	virtual TSharedPtr<SWidget> GetAsWidget() = 0;
};

struct FSlateIMChild : private TUnion<TYPE_OF_NULLPTR, TSharedRef<SWidget>, TSharedRef<ISlateIMChild>>
{
	FSlateIMChild(TYPE_OF_NULLPTR = nullptr) : TUnion(nullptr)
	{}
	FSlateIMChild(const TSharedRef<SWidget>& Widget) : TUnion(Widget)
	{}
	FSlateIMChild(const TSharedRef<ISlateIMChild>& Entry) : TUnion(Entry)
	{}

	TSharedPtr<SWidget> GetWidget() const
	{
		if (HasSubtype<TSharedRef<SWidget>>())
		{
			return GetSubtype<TSharedRef<SWidget>>();
		}
		
		return nullptr;
	}
	
	TSharedRef<SWidget> GetWidgetRef() const
	{
		TSharedPtr<SWidget> Widget = GetWidget();
		return Widget ? Widget.ToSharedRef() : SNullWidget::NullWidget;
	}

	template<typename WidgetType>
	TSharedPtr<WidgetType> GetWidget() const
	{
		TSharedPtr<SWidget> Widget = GetWidget();
		
		if (Widget && Widget->GetWidgetClass().GetWidgetType() == WidgetType::StaticWidgetClass().GetWidgetType())
		{
			return StaticCastSharedPtr<WidgetType>(Widget);
		}
		return nullptr;
	}

	TSharedPtr<ISlateIMChild> GetChild() const
	{
		if (HasSubtype<TSharedRef<ISlateIMChild>>())
		{
			return GetSubtype<TSharedRef<ISlateIMChild>>();
		}
		
		return nullptr;
	}

	template<typename ChildType>
	TSharedPtr<ChildType> GetChild() const
	{
		TSharedPtr<ISlateIMChild> Child = GetChild();
		if (Child && Child->IsA<ChildType>())
		{
			return StaticCastSharedPtr<ChildType>(Child);
		}

		return nullptr;
	}
};