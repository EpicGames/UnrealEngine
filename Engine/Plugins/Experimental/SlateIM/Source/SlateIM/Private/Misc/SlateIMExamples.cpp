// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_SLATEIM_EXAMPLES
#include "SlateIMExamples.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/App.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "SlateIM.h"
#include "SlateIMLogging.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
#include "SLevelViewport.h"
#endif

#if WITH_ENGINE
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#endif

namespace SlateIM::Private
{
	struct FExposedSlateStyle : public FSlateStyleSet
	{
	private:
		FExposedSlateStyle()
			: FSlateStyleSet(TEXT("ExposedSlateStyleSet"))
		{}
		
	public:
		TArray<FName> GetBrushStyleKeys(const FString& SearchString) const
		{
			TArray<FName> Keys;
			BrushResources.GenerateKeyArray(Keys);

			if (SearchString.IsEmpty())
			{
				return Keys;
			}
			
			return Keys.FilterByPredicate([&SearchString](const FName& Key)
			{
				return Key.ToString().ToLower().Contains(SearchString);
			});
		}

		template<typename WidgetStyle>
		TMap<FName, const WidgetStyle*> GetWidgetStyles(const FString& SearchString) const
		{
			TMap<FName, const WidgetStyle*> Styles;

			for (const auto& StylePair : WidgetStyleValues)
			{
				if (StylePair.Value->GetTypeName() == WidgetStyle::TypeName)
				{
					if (SearchString.IsEmpty() || StylePair.Key.ToString().ToLower().Contains(SearchString))
					{
						Styles.Add(StylePair.Key, static_cast<const WidgetStyle*>(&StylePair.Value.Get()));
					}
				}
			}

			return Styles;
		}
	};

	FLinearColor GetKeyStateColor(const FKey& Key)
	{
		if (SlateIM::IsKeyPressed(Key))
		{
			return FLinearColor::Green;
		}
		else if (SlateIM::IsKeyHeld(Key))
		{
			return FLinearColor::Blue;	
		}
		else if (SlateIM::IsKeyReleased(Key))
		{
			return FLinearColor::Red;	
		}

		return FLinearColor(0.1f, 0.1f, 0.1f);
	}
}

void FSlateIMTestWidget::Draw()
{
	double LastTime = CurrentTime;
	CurrentTime = 0;
	FScopedDurationTimer Timer(CurrentTime);

	TimeSinceLastUpdate += FApp::GetDeltaTime();

	SCOPED_NAMED_EVENT_TEXT("FSlateIMTestWidget::Draw", FColorList::Goldenrod);
	constexpr bool bAbsorbMouse = false;

	SlateIM::HAlign(HAlign_Fill);
	SlateIM::BeginVerticalStack();

	SlateIM::BeginMenuBar();
	SlateIM::AddMenuBarEntry(TEXT("Test"));
	SlateIM::AddMenuButton(TEXT("Option 1"));
	SlateIM::EndMenuBar();

	SlateIM::BeginBorder(FAppStyle::GetBrush("ToolPanel.GroupBorder"), {.bAbsorbMouse = bAbsorbMouse});
	// Basic perf measurement, outside of the scrollbox so that it's "pinned"
	if (TimeSinceLastUpdate > 0.5f)
	{
		TimeText = FString::Printf(TEXT("%.3f ms"), LastTime * 1000);
		TimeSinceLastUpdate = 0;
	}

	SlateIM::BeginHorizontalStack();
	{
#if WITH_ENGINE
		if (GEngine && SlateIM::Button(TEXT("Open Style Browser")))
		{
			GEngine->Exec(nullptr, TEXT("SlateIM.ToggleSlateStyleBrowser"));
		}
#endif
		
		SlateIM::Fill();
		SlateIM::HAlign(HAlign_Right);
		SlateIM::Padding({ 5.f, 5.f, 5.f, 0.f });
		SlateIM::Text(TimeText);
	}
	SlateIM::EndHorizontalStack();
	
	SlateIM::Fill();
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::BeginTabGroup(TEXT("ExampleContent"));
	SlateIM::BeginTabStack();
	if (SlateIM::BeginTab(TEXT("Basics")))
	{
		DrawBasics();
	}
	SlateIM::EndTab();

	if (SlateIM::BeginTab(TEXT("Lists and Tables")))
	{
		DrawTables();
	}
	SlateIM::EndTab();

	if (SlateIM::BeginTab(TEXT("Graphs")))
	{
		DrawGraphs();
	}
	SlateIM::EndTab();

	if (SlateIM::BeginTab(TEXT("Inputs")))
	{
		DrawInputs();
	}
	SlateIM::EndTab();

	if (SlateIM::BeginTab(TEXT("Tabs")))
	{
		DrawTabs();
	}
	SlateIM::EndTab();

	if (SlateIM::BeginTab(TEXT("Filter")))
	{
		DrawFilter();
	}
	SlateIM::EndTab();
	SlateIM::EndTabStack();
	SlateIM::EndTabGroup();
	SlateIM::EndBorder();

	SlateIM::EndVerticalStack();
}

void FSlateIMTestWidget::DrawBasics()
{
	SCOPED_NAMED_EVENT_TEXT("Basics", FColorList::Goldenrod);
	SlateIM::Fill();
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::Padding({ 5.f });
	SlateIM::BeginScrollBox();
	{
		{
			SCOPED_NAMED_EVENT_TEXT("Input Widget Examples", FColorList::Goldenrod);
			// Button Examples
			constexpr FStringView ClickText = TEXT("Click Me!");
			if (SlateIM::Button(ClickText))
			{
				UE_LOGF(LogSlateIM, Log, "Button was clicked");
			}

			SlateIM::HAlign(HAlign_Fill);
			if (SlateIM::Button(TEXT("Filled Button")))
			{
				UE_LOGF(LogSlateIM, Log, "Button was clicked");
			}

			{
				SlateIM::BeginHorizontalStack();
				SlateIM::Button(TEXT("Button you can disable"), bIsButtonEnabled);
				SlateIM::CheckBox(TEXT("Enabled"), bIsButtonEnabled);
				SlateIM::EndHorizontalStack();
			}

			{
				SlateIM::BeginVerticalStack();
				// EditableText Example
				{
					SlateIM::BeginHorizontalStack();
					SlateIM::EditableText(ComboItemToAdd, {.HintText = TEXT("Add Combo Item")});
					if (SlateIM::IsFocused(SlateIM::EFocusDepth::IncludingDescendants))
					{
						SlateIM::BeginPopUp();
						SlateIM::Text(TEXT("Enter the value of a new item to add to the combo box"));
						SlateIM::EndPopUp();
					}

					bool bDisableAddButton = ComboItemToAdd.IsEmpty();
					if (bDisableAddButton)
					{
						SlateIM::BeginDisabledState();
					}

					if (SlateIM::Button(TEXT("Add Combo Item")))
					{
						bRefreshComboItems = true;
						ComboBoxItems.Add(MoveTemp(ComboItemToAdd));
						ComboItemToAdd.Reset();
					}

					if (bDisableAddButton)
					{
						SlateIM::EndDisabledState();
					}
					SlateIM::EndHorizontalStack();
				}
				SlateIM::EndVerticalStack();
			}

			// CheckBox examples
			SlateIM::CheckBox(CheckState, {.Label = TEXT("Check Box")});
			if (CheckState)
			{
				SlateIM::Text(TEXT("Basic Text"));
				SlateIM::Text(TEXT("Text With Color"), {.Color = FLinearColor::Green});
				SlateIM::Text(TEXT("Text With style color"), {.Color = FStyleColors::Primary});
			}
	
			if (SlateIM::CheckBox(CheckStateEnum, {.Label = TEXT("Undetermined Check Box")}))
			{
				UE_LOGF(LogSlateIM, Log, "Check Box State Changed");
			}

			// ComboBox example
			SlateIM::BeginHorizontalStack();
			if (SlateIM::ComboBox(ComboBoxItems, SelectedItemIndex, {.bForceRefresh = bRefreshComboItems}))
			{
				if (ComboBoxItems.IsValidIndex(SelectedItemIndex))
				{
					UE_LOGF(LogSlateIM, Log, "Combo Box Item %ls chosen", *ComboBoxItems[SelectedItemIndex]);
				}
			}
			SlateIM::Text(TEXT("Combo Box"));
			SlateIM::EndHorizontalStack();

			SlateIM::BeginHorizontalStack();
			if (SlateIM::ComboBox(ComboBoxItems, SearchableComboBoxSelectedItemIndex, {.bForceRefresh = bRefreshComboItems, .bSearchable = true}))
			{
				if (ComboBoxItems.IsValidIndex(SearchableComboBoxSelectedItemIndex))
				{
					UE_LOGF(LogSlateIM, Log, "Searchable Combo Box Item %ls chosen", *ComboBoxItems[SearchableComboBoxSelectedItemIndex]);
				}
			}
			SlateIM::Text(TEXT("Searchable Combo Box"));
			SlateIM::EndHorizontalStack();
		}

		{
			SCOPED_NAMED_EVENT_TEXT("Center Texture and Buttons", FColorList::Goldenrod);
			// Centered alignment example
			SlateIM::HAlign(HAlign_Center);
			SlateIM::BeginHorizontalStack();

#if WITH_ENGINE
			// Texture Example
			const int32 CurrentSeconds = FApp::GetCurrentTime();
			SlateIM::VAlign(VAlign_Center);
			UTexture2D* Texture = ((CurrentSeconds % 2) == 0)
				? GreenIcon.LoadSynchronous()
				: RedIcon.LoadSynchronous();
			SlateIM::Image(Texture);
#endif
	
			SlateIM::BeginVerticalStack();
			SlateIM::Button(TEXT("Button 1"));
			SlateIM::Button(TEXT("Button 2"));
			SlateIM::Button(TEXT("Button 3"));
			SlateIM::EndVerticalStack();

			SlateIM::BeginVerticalStack();
			SlateIM::Button(TEXT("Button 4"));
			SlateIM::Button(TEXT("Button 5"));
			SlateIM::Button(TEXT("Button 6"));
			SlateIM::EndVerticalStack();

			// SelectionList example
			SlateIM::SelectionList(ComboBoxItems, SelectedItem, {.bForceRefresh = bRefreshComboItems});

			SlateIM::EndHorizontalStack();

			{
				SCOPED_NAMED_EVENT_TEXT("Style image examples", FColorList::Goldenrod);
				// Style image examples
				SlateIM::BeginHorizontalStack();
				SlateIM::Image("AppIcon");
				SlateIM::Padding(FMargin(20, 10, 0, 0));
				SlateIM::Image("Icons.ErrorWithColor");
				SlateIM::Padding(FMargin(SliderVal, 10, 0, 0));
				SlateIM::Image("Icons.WarningWithColor");
				SlateIM::Padding(FMargin(SliderVal, 10, 0, 0));
				SlateIM::Image("Icons.InfoWithColor");
				SlateIM::Padding(FMargin(SliderVal, 10, 0, 0));
				SlateIM::Image("Icons.SuccessWithColor");
				SlateIM::EndHorizontalStack();
			}
		}

		// Slider, ProgressBar, SpinBox
		SlateIM::HAlign(HAlign_Fill);
		SlateIM::BeginHorizontalStack();
		{
			SCOPED_NAMED_EVENT_TEXT("Slider, ProgressBar, SpinBox", FColorList::Goldenrod);
			SlateIM::BeginVerticalStack();
			{
				// Slider example
				if (SlateIM::Slider(SliderVal, {.Max = SliderMax, .Step = 1.f}))
				{
					UE_LOGF(LogSlateIM, Log, "Slider Value Changed [%f]", SliderVal);
				}

				// ProgressBar example
				SlateIM::ProgressBar(IntValue / static_cast<float>(IntMax));

				// SpinBox Examples
				{
					SlateIM::Padding(FMargin(0, 10, 0, 5));
					SlateIM::SpinBox(SliderVal, {.Min = 0.0f, .Max = SliderMax});

					SlateIM::Padding(FMargin(0, 10, 0, 5));
					SlateIM::SpinBox(IntValue, {.Min = 0, .Max = IntMax});
				}
			}
			SlateIM::EndVerticalStack();

			SlateIM::Fill();
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::VAlign(VAlign_Fill);
			if (SlateIM::Button(TEXT("Reset Values")))
			{
				IntValue = 50;
				SliderVal = 5.f;
			}
		}
		SlateIM::EndHorizontalStack();

		{
			SCOPED_NAMED_EVENT_TEXT("ToolTip example", FColorList::Goldenrod);
			// ToolTip example
			SlateIM::SetToolTip(TEXT("This Is a Tool Tip"));
			SlateIM::BeginHorizontalStack();
			SlateIM::Text(TEXT("Tool Tip Testing:"));
			SlateIM::Image("AppIcon");
			SlateIM::EndHorizontalStack();
		}
	
		{
			SCOPED_NAMED_EVENT_TEXT("PopUp example", FColorList::Goldenrod);
			// PopUp example
			SlateIM::Padding(0);
			SlateIM::BeginHorizontalStack();
			SlateIM::Text(TEXT("Hover here to Show a floating popup"));
			if (SlateIM::IsHovered())
			{
				SlateIM::BeginPopUp();
				SlateIM::Text(TEXT("Pop Up Test:"));
				SlateIM::Image("AppIcon");
				SlateIM::EndPopUp();
			}
			SlateIM::EndHorizontalStack();
		}
	
		{
			SCOPED_NAMED_EVENT_TEXT("DisabledState example", FColorList::Goldenrod);
			// DisabledState example
			SlateIM::CheckBox(bShouldBeDisabled, {.Label = TEXT("Disable Everything Below Me")});
			if (bShouldBeDisabled)
			{
				SlateIM::BeginDisabledState();
			}
		}

		// ContextMenu examples
		{
			SCOPED_NAMED_EVENT_TEXT("ContextMenu examples", FColorList::Goldenrod);
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::Text(TEXT("Context Menu Test"));
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::BeginContextMenuAnchor();
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::Text(TEXT("Right Click here to show a menu"));

			// This part is only shown if the menu is open
			SlateIM::AddMenuSection(TEXT("Menu Section 1"));
			if (SlateIM::AddMenuButton(TEXT("Menu Item 1"), {.ToolTipText = TEXT("Menu Item Tool Tip 1")}))
			{
				UE_LOGF(LogSlateIM, Log, "Menu Item One menu option clicked");
			}

			SlateIM::AddMenuButton(TEXT("Menu Item 2"), {.ToolTipText = TEXT("Menu Item Tool Tip 2")});
			SlateIM::AddMenuButton(TEXT("Menu Item 3"), {.ToolTipText = TEXT("Menu Item Tool Tip 3")});
			SlateIM::AddMenuButton(TEXT("Menu Item 4"), {.ToolTipText = TEXT("Menu Item Tool Tip 4")});

			SlateIM::AddMenuCheckButton(TEXT("Menu Item With Check"), MenuCheckState, {.ToolTipText = TEXT("Click to toggle check")});

			if (SlateIM::AddMenuToggleButton(TEXT("Menu Item With Toggle"), MenuToggleState, {.ToolTipText = TEXT("Toggle this box")}))
			{
				UE_LOGF(LogSlateIM, Log, "Menu Item With Toggle clicked");
			}

			SlateIM::AddMenuSeparator();
			SlateIM::BeginSubMenu(TEXT("Sub Menu"));
			SlateIM::AddMenuButton(TEXT("SubMenu Item 1"), {.ToolTipText = TEXT("Menu Item Tool Tip 1")});
			SlateIM::AddMenuButton(TEXT("SubMenu Item 2"), {.ToolTipText = TEXT("Menu Item Tool Tip 2")});
			SlateIM::AddMenuButton(TEXT("SubMenu Item 3"), {.ToolTipText = TEXT("Menu Item Tool Tip 3")});
			SlateIM::AddMenuButton(TEXT("SubMenu Item 4"), {.ToolTipText = TEXT("Menu Item Tool Tip 4")});
			SlateIM::EndSubMenu();
			SlateIM::AddMenuButton(TEXT("Menu Item 5"), {.ToolTipText = TEXT("A Menu Item between 2 SubMenus")});
			SlateIM::AddMenuButton(TEXT("Menu Item 6"), {.ToolTipText = TEXT("A Menu Item between 2 SubMenus")});
			SlateIM::BeginSubMenu(TEXT("Sub Menu 2"));
			SlateIM::AddMenuButton(TEXT("SubMenu2 Item 1"), {.ToolTipText = TEXT("Menu Item Tool Tip 1")});
			SlateIM::AddMenuButton(TEXT("SubMenu2 Item 2"), {.ToolTipText = TEXT("Menu Item Tool Tip 2")});
			SlateIM::AddMenuButton(TEXT("SubMenu2 Item 3"), {.ToolTipText = TEXT("Menu Item Tool Tip 3")});
			SlateIM::AddMenuButton(TEXT("SubMenu2 Item 4"), {.ToolTipText = TEXT("Menu Item Tool Tip 4")});
			SlateIM::EndSubMenu();
			SlateIM::EndContextMenuAnchor();
		}


		// Modal Examples
		{
			SCOPED_NAMED_EVENT_TEXT("Modal Examples", FColorList::Goldenrod);
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::Text(TEXT("Open Modal Dialog of Type:"));

			// Wrap example
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::BeginHorizontalWrap();
			if (SlateIM::Button(TEXT("Ok")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::Ok, TEXT("Ok?"));
			}
			if (SlateIM::Button(TEXT("YesNo")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::YesNo, TEXT("YesNo?"));
			}
			if (SlateIM::Button(TEXT("OkCancel")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::OkCancel, TEXT("OkCancel?"));
			}
			if (SlateIM::Button(TEXT("YesNoCancel")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::YesNoCancel, TEXT("YesNoCancel?"));
			}
			if (SlateIM::Button(TEXT("CancelRetryContinue")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::CancelRetryContinue, TEXT("CancelRetryContinue?"));
			}
			if (SlateIM::Button(TEXT("YesNoYesAllNoAll")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::YesNoYesAllNoAll, TEXT("YesNoYesAllNoAll?"));
			}
			if (SlateIM::Button(TEXT("YesNoYesAllNoAllCancel")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::YesNoYesAllNoAllCancel, TEXT("YesNoYesAllNoAllCancel?"));
			}
			if (SlateIM::Button(TEXT("YesNoYesAll")))
			{
				DialogResult = SlateIM::ModalDialog(EAppMsgType::YesNoYesAll, TEXT("YesNoYesAll?"));
			}
			SlateIM::EndHorizontalWrap();

			if (DialogResult.IsSet())
			{
				SlateIM::HAlign(HAlign_Fill);
				SlateIM::Text(TEXT("Dialog Result:"));
				SlateIM::HAlign(HAlign_Fill);
				switch (DialogResult.GetValue())
				{
				case EAppReturnType::No:
					SlateIM::Text(TEXT("No"));
					break;
				case EAppReturnType::Yes:
					SlateIM::Text(TEXT("Yes"));
					break;
				case EAppReturnType::YesAll:
					SlateIM::Text(TEXT("Yes to All"));
					break;
				case EAppReturnType::NoAll:
					SlateIM::Text(TEXT("No to All"));
					break;
				case EAppReturnType::Cancel:
					SlateIM::Text(TEXT("Cancel"));
					break;
				case EAppReturnType::Ok:
					SlateIM::Text(TEXT("Ok"));
					break;
				case EAppReturnType::Retry:
					SlateIM::Text(TEXT("Retry"));
					break;
				case EAppReturnType::Continue:
					SlateIM::Text(TEXT("Continue"));
					break;
				default:
					SlateIM::Text(TEXT("UNHANDLED RESULT"));
					break;
				}
		
				if (SlateIM::Button(TEXT("Reset")))
				{
					DialogResult.Reset();
				}
			}
			else
			{
				SlateIM::HAlign(HAlign_Fill);
				SlateIM::Text(TEXT("No Dialog Result"));
			}
		}

		if (bShouldBeDisabled)
		{
			SlateIM::EndDisabledState();
		}
	}
	SlateIM::EndScrollBox();

	bRefreshComboItems = false;
}

void FSlateIMTestWidget::DrawTables()
{
	SCOPED_NAMED_EVENT_TEXT("Lists and Tables", FColorList::Goldenrod);
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::BeginScrollBox();
	{
		int32 NumTableItems = 0;
		SlateIM::BeginHorizontalStack();
		{
			SlateIM::Text(TEXT("Num Items "));
			SlateIM::MinWidth(50.f);
			if (SlateIM::EditableText(NumItemsText))
			{
				LiveNumItems = FCString::Atoi(*NumItemsText);
			}

			if (SlateIM::Button(TEXT("Regenerate")))
			{
				NumItems = FCString::Atoi(*NumItemsText);
			}

			NumTableItems = NumItems;
			SlateIM::CheckBox(bShouldLiveUpdateTable, {.Label = TEXT("Live Update Table?")});
			if (bShouldLiveUpdateTable)
			{
				NumTableItems = LiveNumItems;
			}
		}
		SlateIM::EndHorizontalStack();

		SlateIM::VAlign(VAlign_Fill);
		SlateIM::HAlign(HAlign_Fill);
		SlateIM::BeginHorizontalStack();

		// ScrollBox example
		{
			SCOPED_NAMED_EVENT_TEXT("ScrollBox example", FColorList::Goldenrod);
			SlateIM::AutoSize();
			SlateIM::MaxHeight(200.f);
			SlateIM::BeginScrollBox();
			for (int32 i = 0; i < NumItems; ++i)
			{
				// New row
				SlateIM::BeginHorizontalStack();
				{
					SlateIM::Padding(FMargin(5, 0));
					SlateIM::VAlign(VAlign_Center);// Centers the button in the row
					SlateIM::Text(FString::Printf(TEXT("Item %d/%d"), i + 1, NumItems), {.Color = FColor::MakeRedToGreenColorFromScalar(static_cast<float>(i) / NumItems)});
					SlateIM::Padding(FMargin(0));

					if (SlateIM::Button(TEXT("Click")))
					{
						UE_LOGF(LogSlateIM, Log, "Button %d clicked", i + 1);
					}
				}
				SlateIM::EndHorizontalStack();
			}
			SlateIM::EndScrollBox();
		}

		// Spacer Example
		SlateIM::Spacer({ 20, 1 });

		// Table Example
		{
			SCOPED_NAMED_EVENT_TEXT("Table Example", FColorList::Goldenrod);
			SlateIM::Fill();
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::MaxHeight(200.f);
			SlateIM::BeginTable({ .SelectionMode = ESelectionMode::Single });
			SlateIM::BeginTableHeader();
			SlateIM::NextTableColumn(TEXT("Item"));
			// Makes clickable area fill the column header
			SlateIM::Maximize();
			SlateIM::Padding(FMargin(0));
			SlateIM::BeginContextMenuAnchor();
			SlateIM::Text(TEXT("Item"));
			SlateIM::AddMenuSection(TEXT("Menu"));
			SlateIM::AddMenuButton(TEXT("Click me!"));
			SlateIM::EndContextMenuAnchor();
			SlateIM::InitialTableColumnWidth(80.f);
			SlateIM::Maximize();
			SlateIM::HAlign(HAlign_Center);
			SlateIM::AddTableColumn(TEXT("Button"), {.Label = TEXT("Button")});
			SlateIM::EndTableHeader();
			SlateIM::BeginTableBody();
			for (int32 i = 0; i < NumTableItems; ++i)
			{
				bool bIsSelected = false;

				if (SlateIM::NextTableCell(&bIsSelected))
				{
					SlateIM::Padding(FMargin(5, 0));
					SlateIM::Fill();
					SlateIM::VAlign(VAlign_Center);

					if (!bIsSelected)
					{
						SlateIM::Text(FString::Printf(TEXT("Item %d/%d"), i + 1, NumTableItems), {.Color = FColor::MakeRedToGreenColorFromScalar(static_cast<float>(i) / NumTableItems)});
					}
					else
					{
						SlateIM::Text(FString::Printf(TEXT("Item %d/%d [Selected]"), i + 1, NumTableItems), {.Color = FColor::MakeRedToGreenColorFromScalar(static_cast<float>(i) / NumTableItems)});
					}
				}

				if (SlateIM::NextTableCell())
				{
					SlateIM::Padding(FMargin(0));
					SlateIM::HAlign(HAlign_Center);
					if (SlateIM::Button(TEXT("Click")))
					{
						UE_LOGF(LogSlateIM, Log, "Table Button %d clicked", i + 1);
					}
				}
			}
			SlateIM::EndTableBody();
			SlateIM::EndTable();
		}
		SlateIM::EndHorizontalStack();
		
		// Tree Example
		{
			SlateIM::CheckBox(bShouldAddCanada, {.Label = TEXT("Show Canada?")});

			SCOPED_NAMED_EVENT_TEXT("Tree Example", FColorList::Goldenrod);
			const FTableRowStyle* TableRowStyle = &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");
			SlateIM::MinWidth(500.f);
			SlateIM::MinHeight(200.f);
			SlateIM::MaxHeight(200.f);
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::VAlign(VAlign_Fill);
			SlateIM::BeginTable({.RowStyle = TableRowStyle});
			SlateIM::AddTableColumn(TEXT("Name"), {.Label = TEXT("Name")});
			SlateIM::AddTableColumn(TEXT("Type"), {.Label = TEXT("Type")});
			{
				if (SlateIM::NextTableCell())
				{
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Text(TEXT("Antarctica"));
				}
				if (SlateIM::NextTableCell())
				{
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Text(TEXT("Continent"));
				}
			
				if (SlateIM::NextTableCell())
				{
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Text(TEXT("North America"));
				}
				if (SlateIM::NextTableCell())
				{
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Text(TEXT("Continent"));
				}
				if (SlateIM::BeginTableRowChildren())
				{
					if (bShouldAddCanada)
					{
						int32 ChildRowId = 1;

						if (SlateIM::NextTableCell())
						{
							SlateIM::VAlign(VAlign_Center);
							SlateIM::Text(TEXT("Canada"));
						}
						if (SlateIM::NextTableCell())
						{
							SlateIM::VAlign(VAlign_Center);
							SlateIM::Text(TEXT("Country"));
						}
						static constexpr FLazyName CanadaChildren = TEXT("CanadaChildren");
						if (SlateIM::BeginTableRowChildren({.ParentRowId = GetTypeHash(CanadaChildren)}))
						{
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("British Columbia"));
							}
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("Province"));
							}
							if (SlateIM::BeginTableRowChildren({.ParentRowId = static_cast<uint32>(ChildRowId++)}))
							{
								if (SlateIM::NextTableCell())
								{
									SlateIM::VAlign(VAlign_Center);
									SlateIM::Text(TEXT("Vancouver"));
								}
								if (SlateIM::NextTableCell())
								{
									SlateIM::VAlign(VAlign_Center);
									SlateIM::Text(TEXT("City"));
								}
							}
							SlateIM::EndTableRowChildren();

							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("Quebec"));
							}
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("Province"));
							}
							if (SlateIM::BeginTableRowChildren({.ParentRowId = static_cast<uint32>(ChildRowId++)}))
							{
								if (SlateIM::NextTableCell())
								{
									SlateIM::VAlign(VAlign_Center);
									SlateIM::Text(TEXT("Montreal"));
								}
								if (SlateIM::NextTableCell())
								{
									SlateIM::VAlign(VAlign_Center);
									SlateIM::Text(TEXT("City"));
								}
							}
							SlateIM::EndTableRowChildren();
						}
						SlateIM::EndTableRowChildren();
					}

					if (SlateIM::NextTableCell())
					{
						SlateIM::VAlign(VAlign_Center);
						SlateIM::Text(TEXT("United States"));
					}
					if (SlateIM::NextTableCell())
					{
						SlateIM::VAlign(VAlign_Center);
						SlateIM::Text(TEXT("Country"));
					}
#if WITH_ENGINE
					constexpr uint32 RowID = UE::HashStringFNV1a32("UnitedStatesChildren");
#else
					constexpr uint32 RowID = 0;
#endif

					if (SlateIM::BeginTableRowChildren({.ParentRowId = RowID}))
					{
						if (SlateIM::NextTableCell())
						{
							SlateIM::VAlign(VAlign_Center);
							SlateIM::Text(TEXT("North Carolina"));
						}
						if (SlateIM::NextTableCell())
						{
							SlateIM::VAlign(VAlign_Center);
							SlateIM::Text(TEXT("State"));
						}
						if (SlateIM::BeginTableRowChildren())
						{
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("Cary"));
							}
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("City"));
							}
						}
						SlateIM::EndTableRowChildren();
					
						if (SlateIM::NextTableCell())
						{
							SlateIM::VAlign(VAlign_Center);
							SlateIM::Text(TEXT("Washington"));
						}
						if (SlateIM::NextTableCell())
						{
							SlateIM::VAlign(VAlign_Center);
							SlateIM::Text(TEXT("State"));
						}
						if (SlateIM::BeginTableRowChildren())
						{
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("Bellevue"));
							}
							if (SlateIM::NextTableCell())
							{
								SlateIM::VAlign(VAlign_Center);
								SlateIM::Text(TEXT("City"));
							}
						}
						SlateIM::EndTableRowChildren();
					}
					SlateIM::EndTableRowChildren();
				}
				SlateIM::EndTableRowChildren();
			}
		
			SlateIM::EndTable();
		}

		// Dynamically added/removed child row
		{
			SlateIM::CheckBox(bShouldAddChildRow, {.Label = TEXT("Show Child?")});

			SlateIM::Fill();
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::BeginTable();
			SlateIM::AddTableColumn(TEXT("Column"), {.Label = TEXT("Column")});

			if (SlateIM::NextTableCell())
			{
				SlateIM::HAlign(HAlign_Fill);
				SlateIM::Text(TEXT("First Row"));
			}
		
			if (SlateIM::NextTableCell())
			{
				SlateIM::HAlign(HAlign_Fill);
				SlateIM::Text(TEXT("Second Row"));
			}

			if (bShouldAddChildRow)
			{
				if (SlateIM::BeginTableRowChildren())
				{
					if (SlateIM::NextTableCell())
					{
						SlateIM::HAlign(HAlign_Fill);
						SlateIM::Text(TEXT("Child Row"));
					}
				}
				SlateIM::EndTableRowChildren();
			}
			SlateIM::EndTable();
		}

		// Dynamic add/remove rows and columns
		{
			constexpr int32 MaxColumnCount = 10;
			static TStaticArray<bool, MaxColumnCount> Columns = {true};
			constexpr int32 MaxRowCount = 10;
			static TStaticArray<bool, MaxRowCount> Rows = {true};

			auto GetColumnName = [] (const int32 Idx) { return FString().AppendChar('A' + Idx); };
			auto GetRowName = [] (const int32 Idx) { return FString::FromInt(Idx); };

			SlateIM::BeginHorizontalStack();
			SlateIM::Text(TEXT("Columns:"));
			for (int32 i = 0; i < Columns.Num(); ++i)
			{
				SlateIM::CheckBox(Columns[i], {.Label = GetColumnName(i)});
			}
			SlateIM::EndHorizontalStack();

			SlateIM::BeginHorizontalStack();
			SlateIM::Text(TEXT("Rows:"));
			for (int32 i = 0; i < Rows.Num(); ++i)
			{
				SlateIM::CheckBox(Rows[i], {.Label = GetRowName(i)});
			}
			SlateIM::EndHorizontalStack();

			SlateIM::MinHeight(200.f);
			SlateIM::MaxHeight(200.f);
			SlateIM::Fill();
			SlateIM::HAlign(HAlign_Fill);
			SlateIM::VAlign(VAlign_Fill);
			SlateIM::BeginTable();
			for (int32 i = 0; i < Columns.Num(); ++i)
			{
				if (Columns[i])
				{
					SlateIM::AddTableColumn(FName(GetColumnName(i)), {.Label = GetColumnName(i)});
				}
			}
			for (int32 i = 0; i < Rows.Num(); ++i)
			{
				if (Rows[i])
				{
					for (int32 j = 0; j < Columns.Num(); ++j)
					{
						if (Columns[j])
						{
							if (SlateIM::NextTableCell())
							{
								SlateIM::Text(FString::Printf(TEXT("%s%s"), *GetColumnName(j), *GetRowName(i)));
							}
						}
					}
				}
			}
			SlateIM::EndTable();
		}
	}
	SlateIM::EndScrollBox();
}

void FSlateIMTestWidget::DrawGraphs()
{
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::Padding(0);
	SlateIM::BeginVerticalStack();

	SCOPED_NAMED_EVENT_TEXT("Graphs", FColorList::Goldenrod);
	SlateIM::BeginHorizontalStack();
	{
		if (SquareGraphValues.Num() >= 100)
		{
			SquareGraphValues.PopFront();
		}
		const double SquareValue = (GFrameCounter / 10) % 2;
		SquareGraphValues.Emplace(SquareValue);
		SlateIM::Fill();
		SlateIM::MinHeight(200.f);
		SlateIM::BeginGraph();
		SlateIM::GraphLine(SquareGraphValues.Compact(), {.ViewRange = FDoubleRange(0, 1.0), .LineColor = FLinearColor::White, .LineThickness = 3.f});
		SlateIM::EndGraph();

		SlateIM::VAlign(VAlign_Center);
		SlateIM::Text(FString::Printf(TEXT("%0.3lf"), SquareValue));

		const double NextSinX = SinGraphValues.Last().X + 1.0;
		const double NextCosX = CosGraphValues.Last().X + 1.0;
		const double NextTanX = TanGraphValues.Last().X + 1.0;
		if (GFrameCounter % 4 == 0)
		{
			if (SinGraphValues.Num() >= 100)
			{
				SinGraphValues.PopFront();
			}
			SinGraphValues.Emplace(NextSinX, FMath::Sin(NextSinX/4.0));
		
			if (CosGraphValues.Num() >= 100)
			{
				CosGraphValues.PopFront();
			}
			CosGraphValues.Emplace(NextCosX, FMath::Cos(NextCosX/4.0));
		
			if (TanGraphValues.Num() >= 100)
			{
				TanGraphValues.PopFront();
			}
			TanGraphValues.Emplace(NextTanX, FMath::Tan(NextTanX/4.0));
		}
		SlateIM::Fill();
		SlateIM::MinHeight(200.f);
		SlateIM::BeginGraph();
		SlateIM::GraphLine(SinGraphValues.Compact(), {.XViewRange = FDoubleRange(NextSinX - 100.0, NextSinX), .YViewRange = FDoubleRange(-1.5, 1.5), .LineColor = FColor::Orange});
		SlateIM::GraphLine(CosGraphValues.Compact(), {.XViewRange = FDoubleRange(NextCosX - 100.0, NextCosX), .YViewRange = FDoubleRange(-1.5, 1.5), .LineColor = FLinearColor::Green});
		SlateIM::GraphLine(TanGraphValues.Compact(), {.XViewRange = FDoubleRange(NextTanX - 100.0, NextTanX), .YViewRange = FDoubleRange(-1.5, 1.5), .LineColor = FColor::Magenta});
		SlateIM::EndGraph();

		SlateIM::MinWidth(50.f);
		SlateIM::MaxWidth(50.f);
		SlateIM::BeginVerticalStack();
		SlateIM::Fill();
		SlateIM::VAlign(VAlign_Center);
		SlateIM::Text(FString::Printf(TEXT("%0.3lf"), SinGraphValues.Last().Y), {.Color = FColor::Orange});
		SlateIM::Fill();
		SlateIM::VAlign(VAlign_Center);
		SlateIM::Text(FString::Printf(TEXT("%0.3lf"), CosGraphValues.Last().Y), {.Color = FLinearColor::Green});
		SlateIM::Fill();
		SlateIM::VAlign(VAlign_Center);
		SlateIM::Text(FString::Printf(TEXT("%0.3lf"), TanGraphValues.Last().Y), {.Color = FColor::Magenta});
		SlateIM::EndVerticalStack();
	}
	SlateIM::EndHorizontalStack();

#if WITH_ENGINE
	SCOPED_NAMED_EVENT_TEXT("CanvasRenderTarget", FColorList::Goldenrod);

	FCanvasIcon Icon;
	Icon.Texture = GreenIcon.LoadSynchronous();

	SlateIM::Canvas::BeginCanvas(100, 100, { .UpdateType = SlateIM::Canvas::ECanvasUpdateType::Invalidation });

	// Allows for caching of textures and materials after the first draw call.
	if (CanvasRenderCount < 2)
	{
		SlateIM::Canvas::Invalidate();
		++CanvasRenderCount;
	}

	SlateIM::Canvas::DrawPolygon(
		nullptr,
		{ 50, 100 },
		{ 35, 35 },
		36
	);

	SlateIM::Canvas::DrawIcon(Icon, { 18, 65 });
	SlateIM::Canvas::DrawLine({ 0, 60 }, { 100, 60 }, 1.f, FLinearColor::Red);

	SlateIM::Canvas::DrawTexture(
		RedIcon.LoadSynchronous(),
		{ 75, 10 },
		{ 30, 40 },
		{ .UVSize = {0.75, 1} }
	);

	UMaterial* MaterialLeft = TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Script/Engine.Material'/Engine/EngineMaterials/ColorGradingWheel.ColorGradingWheel'"))).LoadSynchronous();
	
	SlateIM::Canvas::DrawMaterial(
		MaterialLeft,
		{ 0, 10 },
		{ 30, 40 },
		{ .UVPosition = {0.25, 0}, .UVSize = {0.75, 1} }
	);

	UFont* EngineFont = TSoftObjectPtr<UFont>(FSoftObjectPath(TEXT("/Script/Engine.Font'/Engine/EngineFonts/RobotoTiny.RobotoTiny'"))).LoadSynchronous();

	SlateIM::Canvas::DrawText(
		EngineFont,
		TEXT("Canvas"),
		{ 50, 15 },
		{
			.Scale = {1.0, 1.0},
			.ShadowColor = FLinearColor(0.5, 0.5, 0.5, 1.f),
			.bCentreX = true
		}
	);

	SlateIM::Canvas::DrawText(
		EngineFont,
		TEXT("Hi!"),
		{ 40, 30 },
		{ 1.5, 1.5 }
	);

	SlateIM::Canvas::DrawBox(
		{ 0, 0 },
		{ 100, 100 },
		2,
		FLinearColor::Blue
	);

	UTexture2D* TriangleTexture = TSoftObjectPtr<UTexture2D>(FSoftObjectPath(TEXT("/Script/Engine.Texture2D'/Engine/EngineMaterials/ColorGradingWheelGradient.ColorGradingWheelGradient'"))).LoadSynchronous();

	TSharedRef<TArray<FCanvasUVTri>> TextureTriangles = MakeShared<TArray<FCanvasUVTri>>();
	{
		FCanvasUVTri& Triangle = TextureTriangles->AddDefaulted_GetRef();
		Triangle.V0_Color = FLinearColor::White;
		Triangle.V0_UV = { 0, 0 };
		Triangle.V0_Pos = { 5, 70 };
		Triangle.V1_Color = FLinearColor::White;
		Triangle.V1_UV = { 0, 1 };
		Triangle.V1_Pos = { 25, 70 };
		Triangle.V2_Color = FLinearColor::White;
		Triangle.V2_UV = { 1, 0 };
		Triangle.V2_Pos = { 5, 90 };
	}

	SlateIM::Canvas::DrawTriangles(TriangleTexture, TextureTriangles);

	UMaterial* TriangleMaterial = TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Script/Engine.Material'/Engine/EngineMaterials/ColorGradingWheel.ColorGradingWheel'"))).LoadSynchronous();

	TSharedRef<TArray<FCanvasUVTri>> MaterialTriangles = MakeShared<TArray<FCanvasUVTri>>();
	{
		FCanvasUVTri& Triangle = MaterialTriangles->AddDefaulted_GetRef();
		Triangle.V0_Color = FLinearColor::White;
		Triangle.V0_UV = { 0, 0 };
		Triangle.V0_Pos = { 75, 70 };
		Triangle.V1_Color = FLinearColor::White;
		Triangle.V1_UV = { 1, 0 };
		Triangle.V1_Pos = { 95, 70 };
		Triangle.V2_Color = FLinearColor::White;
		Triangle.V2_UV = { 1, 1 };
		Triangle.V2_Pos = { 95, 90 };
	}

	SlateIM::Canvas::DrawMaterialTriangles(TriangleMaterial, MaterialTriangles);

	SlateIM::Canvas::EndCanvas();
#endif

	SlateIM::EndVerticalStack();
}

void FSlateIMTestWidget::DrawInputs()
{
	SCOPED_NAMED_EVENT_TEXT("Inputs", FColorList::Goldenrod);
	SlateIM::HAlign(HAlign_Center);
	SlateIM::BeginHorizontalStack();
	{
		SlateIM::BeginVerticalStack();
		SlateIM::BeginHorizontalStack();
		SlateIM::Spacer(FVector2D(50, 50));
		{
			WBrush = FSlateColorBrush(SlateIM::Private::GetKeyStateColor(EKeys::W));
			SlateIM::MinWidth(50.f);
			SlateIM::MinHeight(50.f);
			SlateIM::MaxWidth(50.f);
			SlateIM::MaxHeight(50.f);
			SlateIM::BeginBorder(&WBrush);
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(TEXT("W"), {.Color = FLinearColor::White});
			SlateIM::EndBorder();
		}
		SlateIM::Spacer(FVector2D(50, 50));
		SlateIM::EndHorizontalStack();
		SlateIM::BeginHorizontalStack();
		{
			ABrush = FSlateColorBrush(SlateIM::Private::GetKeyStateColor(EKeys::A));
			SlateIM::MinWidth(50.f);
			SlateIM::MinHeight(50.f);
			SlateIM::MaxWidth(50.f);
			SlateIM::MaxHeight(50.f);
			SlateIM::BeginBorder(&ABrush);
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(TEXT("A"), {.Color = FLinearColor::White});
			SlateIM::EndBorder();
		}
		{
			SBrush = FSlateColorBrush(SlateIM::Private::GetKeyStateColor(EKeys::S));
			SlateIM::MinWidth(50.f);
			SlateIM::MinHeight(50.f);
			SlateIM::MaxWidth(50.f);
			SlateIM::MaxHeight(50.f);
			SlateIM::BeginBorder(&SBrush);
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(TEXT("S"), {.Color = FLinearColor::White});
			SlateIM::EndBorder();
		}
		{
			DBrush = FSlateColorBrush(SlateIM::Private::GetKeyStateColor(EKeys::D));
			SlateIM::MinWidth(50.f);
			SlateIM::MinHeight(50.f);
			SlateIM::MaxWidth(50.f);
			SlateIM::MaxHeight(50.f);
			SlateIM::BeginBorder(&DBrush);
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(TEXT("D"), {.Color = FLinearColor::White});
			SlateIM::EndBorder();
		}
		SlateIM::EndHorizontalStack();
		SlateIM::EndVerticalStack();
	}

	{
		SlateIM::BeginVerticalStack();
		SlateIM::Text(TEXT("Mouse X-value"));
		const float NormalizedAnalogXValue = (1.f + (SlateIM::GetKeyAnalogValue(EKeys::MouseX) / 100.f)) * 0.5f;
		SlateIM::VAlign(VAlign_Center);
		SlateIM::ProgressBar(NormalizedAnalogXValue);
	
		SlateIM::Fill();
		SlateIM::Spacer(FVector2D(1.f, 1.f));
	
		SlateIM::Text(TEXT("Mouse Y-value"));
		const float NormalizedAnalogYValue = (1.f + (SlateIM::GetKeyAnalogValue(EKeys::MouseY) / 100.f)) * 0.5f;
		SlateIM::VAlign(VAlign_Center);
		SlateIM::ProgressBar(NormalizedAnalogYValue);

		SlateIM::BeginHorizontalStack();
		{
			LMBBrush = FSlateColorBrush(SlateIM::Private::GetKeyStateColor(EKeys::LeftMouseButton));
			SlateIM::MinWidth(50.f);
			SlateIM::MinHeight(50.f);
			SlateIM::MaxWidth(50.f);
			SlateIM::MaxHeight(50.f);
			SlateIM::BeginBorder(&LMBBrush);
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(TEXT("LMB"), {.Color = FLinearColor::White});
			SlateIM::EndBorder();
		}
		{
			RMBBrush = FSlateColorBrush(SlateIM::Private::GetKeyStateColor(EKeys::RightMouseButton));
			SlateIM::MinWidth(50.f);
			SlateIM::MinHeight(50.f);
			SlateIM::MaxWidth(50.f);
			SlateIM::MaxHeight(50.f);
			SlateIM::BeginBorder(&RMBBrush);
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Text(TEXT("RMB"), {.Color = FLinearColor::White});
			SlateIM::EndBorder();
		}
		SlateIM::EndHorizontalStack();
		SlateIM::EndVerticalStack();
	}

	{
		SlateIM::BeginVerticalStack();
		SlateIM::Text(TEXT("Right Stick X-value"));
		const float NormalizedAnalogXValue = (1.f + SlateIM::GetKeyAnalogValue(EKeys::Gamepad_RightX)) * 0.5f;
		SlateIM::VAlign(VAlign_Center);
		SlateIM::ProgressBar(NormalizedAnalogXValue);
	
		SlateIM::Fill();
		SlateIM::Spacer(FVector2D(1.f, 1.f));
	
		SlateIM::Text(TEXT("Right Stick Y-value"));
		const float NormalizedAnalogYValue = (1.f + SlateIM::GetKeyAnalogValue(EKeys::Gamepad_RightY)) * 0.5f;
		SlateIM::VAlign(VAlign_Center);
		SlateIM::ProgressBar(NormalizedAnalogYValue);
		SlateIM::EndVerticalStack();
	}
	SlateIM::EndHorizontalStack();
}

void FSlateIMTestWidget::DrawTabs()
{
	SCOPED_NAMED_EVENT_TEXT("Tabs", FColorList::Goldenrod);
	{
		SlateIM::Text(TEXT("Splitters"));
		SlateIM::HAlign(HAlign_Fill);
		SlateIM::VAlign(VAlign_Fill);
		SlateIM::Fill();
		SlateIM::BeginTabGroup(TEXT("SlateIMTabTesting1"));
		{
			SlateIM::BeginTabSplitter(Orient_Horizontal);
			{
				SlateIM::TabSplitterSizeCoefficient(0.3f);
				SlateIM::BeginTabStack();
				{
					if (SlateIM::BeginTab(TEXT("Left Tab"), {.TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Symbols.LeftArrow"))}))
					{
						// Fill so that the text autowrapping updates properly as the splitter is adjusted by the user
						SlateIM::HAlign(HAlign_Fill);
						SlateIM::Text(TEXT("Left Tab taking up 30% of the horizontal space"));
					}
					SlateIM::EndTab();
					if (SlateIM::BeginTab(TEXT("Right Tab"), {.TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Symbols.RightArrow"))}))
					{
						// Fill so that the text autowrapping updates properly as the splitter is adjusted by the user
						SlateIM::HAlign(HAlign_Fill);
						SlateIM::Text(TEXT("Right Tab taking up 30% of the horizontal space"));
					}
					SlateIM::EndTab();
				}
				SlateIM::EndTabStack();
			
				SlateIM::TabSplitterSizeCoefficient(0.7f);
				SlateIM::BeginTabSplitter(Orient_Vertical);
				{
					SlateIM::TabSplitterSizeCoefficient(0.3f);
					SlateIM::BeginTabStack();
					{
						if (SlateIM::BeginTab(TEXT("Tab 1")))
						{
							// Fill so that the text autowrapping updates properly as the splitter is adjusted by the user
							SlateIM::HAlign(HAlign_Fill);
							SlateIM::Text(TEXT("Tab that takes up 30% of the vertical space"));
						}
						SlateIM::EndTab();
					}
					SlateIM::EndTabStack();
				
					SlateIM::TabSplitterSizeCoefficient(0.7f);
					SlateIM::BeginTabStack();
					{
						if (SlateIM::BeginTab(TEXT("Tab 2")))
						{
							// Fill so that the text autowrapping updates properly as the splitter is adjusted by the user
							SlateIM::HAlign(HAlign_Fill);
							SlateIM::Text(TEXT("Tab that takes up 70% of the vertical space"));
						}
						SlateIM::EndTab();
					}
					SlateIM::EndTabStack();
				}
				SlateIM::EndTabSplitter();
			}
			SlateIM::EndTabSplitter();
		}
		SlateIM::EndTabGroup();
	}
	SlateIM::Spacer(FVector2D(1.f, 20.f));
	{
		SlateIM::Text(TEXT("Dynamic Tabs"));
		SlateIM::BeginHorizontalStack();
		if (SlateIM::Button(TEXT("Add Tab")))
		{
			SlateIM::ActivateTab(*FString::Printf(TEXT("DynamicTab%d"), DynamicTabCount));
			++DynamicTabCount;
		}
		if (SlateIM::Button(TEXT("Remove Tab")))
		{
			DynamicTabCount = FMath::Max(0, DynamicTabCount - 1);
		}
		SlateIM::EndHorizontalStack();
		
		SlateIM::HAlign(HAlign_Fill);
		SlateIM::VAlign(VAlign_Fill);
		SlateIM::Fill();
		SlateIM::BeginTabGroup(TEXT("SlateIMTabTesting2"));
		{
			SlateIM::BeginTabStack();
			{
				for (int32 i = 0; i < DynamicTabCount; i++)
				{
					if (SlateIM::BeginTab(*FString::Printf(TEXT("DynamicTab%d"), i), {.TabTitle = FText::Format(INVTEXT("Tab {0}"), FText::AsNumber(i + 1))}))
					{
						SlateIM::Text(FString::Printf(TEXT("Tab %d Content"), i + 1));
					}
					SlateIM::EndTab();
				}
			}
			SlateIM::EndTabStack();
		}
		SlateIM::EndTabGroup();
	}
	SlateIM::Spacer(FVector2D(1.f, 20.f));
	{
		SlateIM::Text(TEXT("Nested Tab Groups"));
		SlateIM::HAlign(HAlign_Fill);
		SlateIM::VAlign(VAlign_Fill);
		SlateIM::Fill();
		SlateIM::BeginTabGroup(TEXT("SlateIMTabTesting3"));
		SlateIM::BeginTabStack();
		{
			if (SlateIM::BeginTab(TEXT("Tab 1")))
			{
				SlateIM::Text(TEXT("See Tab 2 for Nested Tabs"));
			}
			SlateIM::EndTab();

			if (SlateIM::BeginTab(TEXT("Tab 2")))
			{
				SlateIM::Fill();
				SlateIM::HAlign(HAlign_Fill);
				SlateIM::BeginTabGroup(TEXT("SlateIMTabTestingNestedTabs"));
				SlateIM::BeginTabStack();
				{
					if (SlateIM::BeginTab(TEXT("Nested Tab 1")))
					{
						SlateIM::Text(TEXT("Nested Tab 1 Content"));
					}
					SlateIM::EndTab();
			
					if (SlateIM::BeginTab(TEXT("Nested Tab 2")))
					{
						SlateIM::Text(TEXT("Nested Tab 2 Content"));
					}
					SlateIM::EndTab();
			
					if (SlateIM::BeginTab(TEXT("Nested Tab 3")))
					{
						SlateIM::Text(TEXT("Nested Tab 3 Content"));
					}
					SlateIM::EndTab();
				}
				SlateIM::EndTabStack();
				SlateIM::EndTabGroup();
			}
			SlateIM::EndTab();
		}
		SlateIM::EndTabStack();
		SlateIM::EndTabGroup();
	}
}

void FSlateIMTestWidget::DrawFilter()
{
	SCOPED_NAMED_EVENT_TEXT("Filter", FColorList::Goldenrod);
	SlateIM::Fill();
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::Padding({ 5.f });
	SlateIM::BeginVerticalStack();
	{
		static TArray<FString> FruitList = {
			TEXT("Apple"),
			TEXT("Banana"),
			TEXT("Cherry"),
			TEXT("Dragonfruit"),
			TEXT("Elderberry"),
			TEXT("Fig"),
			TEXT("Grape"),
			TEXT("Honeydew"),
			TEXT("Kiwi"),
			TEXT("Lemon")
		};

		if (!FilteredFruitList.IsSet())
		{
			FilteredFruitList = FruitList;
		}

		// Draw the filter input and filter the list
		bool bFruitListChanged = false;
		if (SlateIM::TextFilter(FruitFilter, {.HintText = TEXT("Search fruits...")}))
		{
			FilteredFruitList = FruitFilter.Filter(FruitList);
			bFruitListChanged = true;
		}

		// Display the filtered list
		SlateIM::Fill();
		SlateIM::HAlign(HAlign_Fill);
		int32 SelectedFruitIndex = INDEX_NONE;
		SlateIM::SelectionList(FilteredFruitList.GetValue(), SelectedFruitIndex, {.bForceRefresh = bFruitListChanged});
	}
	SlateIM::EndVerticalStack();
}

void FSlateStyleBrowser::DrawWindow(float DeltaTime)
{
	static const TArray<FString> Options = { TEXT("Option 1"), TEXT("Option 2"), TEXT("Option 3") };
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::Fill();
	SlateIM::BeginVerticalStack();
	SlateIM::EditableText(SearchString, {.HintText = TEXT("Search Styles...")});
	
	SlateIM::HAlign(HAlign_Fill);
	SlateIM::VAlign(VAlign_Fill);
	SlateIM::Fill();
	SlateIM::BeginTable();
	SlateIM::AddTableColumn(TEXT("Name"), {.Label = TEXT("Name")});
	SlateIM::AddTableColumn(TEXT("Preview"), {.Label = TEXT("Preview")});
	SlateIM::InitialTableColumnWidth(80.f);
	SlateIM::AddTableColumn(TEXT("Button"), {.Label = TEXT("Button")});

	const auto& Style = static_cast<const SlateIM::Private::FExposedSlateStyle&>(FAppStyle::Get());
	const FString LowerCaseSearchString = SearchString.ToLower();

	auto DrawNameCell = [](const FName& Key)
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Fill();
			SlateIM::Text(Key.ToString());
		}
	};

	auto DrawCopyCell = [](const FName& Key)
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::SetToolTip(TEXT("Click to copy the style name to your clipboard"));
			SlateIM::HAlign(HAlign_Center);
			SlateIM::VAlign(VAlign_Center);
			SlateIM::Fill();
			if (SlateIM::Button(TEXT("Copy")))
			{
				FPlatformApplicationMisc::ClipboardCopy(*Key.ToString());
			}
		}
	};

	// Brushes
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Brushes"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column

		if (SlateIM::BeginTableRowChildren())
		{
			for (const FName& BrushStyleKey : Style.GetBrushStyleKeys(LowerCaseSearchString))
			{
				if (const FSlateBrush* Brush = Style.GetBrush(BrushStyleKey))
				{
					DrawNameCell(BrushStyleKey);

					if (SlateIM::NextTableCell())
					{
						SlateIM::HAlign(HAlign_Center);
						SlateIM::VAlign(VAlign_Center);
						SlateIM::Fill();
						SlateIM::Image(Brush);
					}
					
					DrawCopyCell(BrushStyleKey);
				}
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// Text Block Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Text Block Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FTextBlockStyle*>& WidgetStyle : Style.GetWidgetStyles<FTextBlockStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::Text(TEXT("The quick brown fox jumps over the lazy dog."), {.Style = WidgetStyle.Value});
				}
			
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// Editable Text Box Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Editable Text Box Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FEditableTextBoxStyle*>& WidgetStyle : Style.GetWidgetStyles<FEditableTextBoxStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::EditableText(PreviewText, {.HintText = TEXT("Hint text..."), .Style = WidgetStyle.Value});
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// Button Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Button Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FButtonStyle*>& WidgetStyle : Style.GetWidgetStyles<FButtonStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::Button(TEXT("Click Me"), {.Style = WidgetStyle.Value});
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// SpinBox Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("SpinBox Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FSpinBoxStyle*>& WidgetStyle : Style.GetWidgetStyles<FSpinBoxStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::SpinBox(SpinBoxValue, {.Min = -100.f, .Max = 100.f, .Style = WidgetStyle.Value});
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// Slider Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Slider Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FSliderStyle*>& WidgetStyle : Style.GetWidgetStyles<FSliderStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::Slider(SliderValue, {.Max = 100.f, .Step = 0.1f, .Style = WidgetStyle.Value});
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// ProgressBar Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Progress Bar Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FProgressBarStyle*>& WidgetStyle : Style.GetWidgetStyles<FProgressBarStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::ProgressBar(SliderValue / 100.f, {.Style = WidgetStyle.Value});
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// ComboBox Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Combo Box Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FComboBoxStyle*>& WidgetStyle : Style.GetWidgetStyles<FComboBoxStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::ComboBox(Options, SelectedComboIndex, {.Style = WidgetStyle.Value});
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}

	// Table View Styles
	{
		if (SlateIM::NextTableCell())
		{
			SlateIM::Text(TEXT("Table View Styles"));
		}
		SlateIM::NextTableCell(); // Skip Preview Column
		SlateIM::NextTableCell(); // Skip Button Column
		if (SlateIM::BeginTableRowChildren())
		{
			for (const TPair<FName, const FTableViewStyle*>& WidgetStyle : Style.GetWidgetStyles<FTableViewStyle>(LowerCaseSearchString))
			{
				DrawNameCell(WidgetStyle.Key);

				if (SlateIM::NextTableCell())
				{
					SlateIM::HAlign(HAlign_Center);
					SlateIM::VAlign(VAlign_Center);
					SlateIM::Fill();
					SlateIM::BeginHorizontalStack();
					{
						SlateIM::HAlign(HAlign_Center);
						SlateIM::VAlign(VAlign_Top);
						SlateIM::Fill();
						SlateIM::SelectionList(Options, SelectedListIndex, {.Style = WidgetStyle.Value});

						SlateIM::HAlign(HAlign_Center);
						SlateIM::VAlign(VAlign_Top);
						SlateIM::Fill();
						SlateIM::BeginTable({.Style = WidgetStyle.Value});
						SlateIM::AddTableColumn(TEXT("Column 1"), {.Label = TEXT("Column 1")});
						SlateIM::AddTableColumn(TEXT("Column 2"), {.Label = TEXT("Column 2")});
						SlateIM::NextTableCell();
						SlateIM::Text(TEXT("Cell 1"));
						SlateIM::NextTableCell();
						SlateIM::Text(TEXT("Cell 2"));
						SlateIM::NextTableCell();
						SlateIM::Text(TEXT("Cell 3"));
						SlateIM::NextTableCell();
						SlateIM::Text(TEXT("Cell 4"));
						if (SlateIM::BeginTableRowChildren())
						{
							SlateIM::NextTableCell();
							SlateIM::Text(TEXT("Cell 5"));
							SlateIM::NextTableCell();
							SlateIM::Text(TEXT("Cell 6"));
						}
						SlateIM::EndTableRowChildren();
						SlateIM::EndTable();
					}
					SlateIM::EndHorizontalStack();
				}
				
				DrawCopyCell(WidgetStyle.Key);
			}
		}
		SlateIM::EndTableRowChildren();
	}
	
	SlateIM::EndTable();
	SlateIM::EndVerticalStack();
}

void FSlateIMTestWindowWidget::DrawWindow(float DeltaTime)
{
	TestWidget.Draw();
}

void FSlateIMTestNomadTabWidget::DrawContent(float DeltaTime)
{
	TestWidget.Draw();
}

#if WITH_ENGINE
void FSlateIMTestViewportWidget::DrawWidget(float DeltaTime)
{
	if (GEngine && GEngine->GameViewport)
	{
		if (SlateIM::BeginViewportRoot("SlateIMTestSuiteViewport", GEngine->GameViewport, {.Layout = Layout}))
		{
			TestWidget.Draw();
		}
		SlateIM::EndRoot();
	}
#if WITH_EDITOR
	else if (GCurrentLevelEditingViewportClient)
	{
		TSharedPtr<SLevelViewport> LevelViewport = StaticCastSharedPtr<SLevelViewport>(GCurrentLevelEditingViewportClient->GetEditorViewportWidget());
		if (LevelViewport.IsValid())
		{
			if (SlateIM::BeginViewportRoot("SlateIMTestSuiteViewport", LevelViewport, {.Layout = Layout}))
			{
				TestWidget.Draw();
			}
			SlateIM::EndRoot();
		}
	}
#endif
}
#endif // WITH_ENGINE
#endif // WITH_SLATEIM_EXAMPLES
