// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/RingBuffer.h"

#if WITH_SLATEIM_EXAMPLES
#include "Misc/SlateIMStringFilter.h"
#include "SlateIM.h"
#include "SlateIMWidgetBase.h"
#include "Styling/SlateTypes.h"

class UTexture2D;

class FSlateIMTestWidget
{
public:
	void Draw();
	void DrawBasics();
	void DrawTables();
	void DrawGraphs();
	void DrawInputs();
	void DrawTabs();
	void DrawFilter();
	
private:
	double CurrentTime = 0;
	double TimeSinceLastUpdate = 0;
	FString TimeText;
	FString ComboItemToAdd;
	bool CheckState = false;
	ECheckBoxState CheckStateEnum = ECheckBoxState::Undetermined;
	int32 SelectedItemIndex = 0;
	int32 SearchableComboBoxSelectedItemIndex = 0;
	float SliderVal = 5.0f;
	float SliderMax = 20.f;
	int32 SelectedItem = INDEX_NONE;
	int32 IntValue = 50;
	int32 IntMax = 100;
	bool bShouldBeDisabled = false;
	bool MenuCheckState = true;
	bool MenuToggleState = true;
	TOptional<EAppReturnType::Type> DialogResult;
	int32 NumItems = 10;
	int32 LiveNumItems = NumItems;
	FString NumItemsText = FString::FromInt(NumItems);
	bool bShouldLiveUpdateTable = false;
	bool bShouldAddCanada = true;
	bool bShouldAddChildRow = false;
	TArray<FString> ComboBoxItems = { TEXT("Option 1"), TEXT("Option 2"), TEXT("Option 3"), TEXT("Option 4") };
	bool bRefreshComboItems = false;
	bool bIsButtonEnabled = true;
	int32 DynamicTabCount = 3;
	FSlateIMStringFilter FruitFilter;
	TOptional<TArray<FString>> FilteredFruitList;
	int32 SelectedKitchenInstrument = INDEX_NONE;
	TRingBuffer<double> SquareGraphValues;
	TRingBuffer<FVector2D> SinGraphValues = { FVector2D(0, FMath::Sin(0.0)) };
	TRingBuffer<FVector2D> CosGraphValues = { FVector2D(0, FMath::Cos(0.0)) };
	TRingBuffer<FVector2D> TanGraphValues = { FVector2D(0, FMath::Tan(0.0)) };

	FSlateBrush WBrush;
	FSlateBrush ABrush;
	FSlateBrush SBrush;
	FSlateBrush DBrush;
	
	FSlateBrush LMBBrush;
	FSlateBrush RMBBrush;

#if WITH_ENGINE
	TSoftObjectPtr<UTexture2D> RedIcon = TSoftObjectPtr<UTexture2D>(FSoftObjectPath(TEXT("/Engine/EngineResources/AICON-Red.AICON-Red")));
	TSoftObjectPtr<UTexture2D> GreenIcon = TSoftObjectPtr<UTexture2D>(FSoftObjectPath(TEXT("/Engine/EngineResources/AICON-Green.AICON-Green")));
	int32 CanvasRenderCount = 0;
#endif
};

class FSlateStyleBrowser : public FSlateIMWindowBase
{
public:
	FSlateStyleBrowser()
		: FSlateIMWindowBase(TEXT("SlateIM Style Browser"), FVector2f(1000, 500), /* Always On Top */ false, TEXT("SlateIM.ToggleSlateStyleBrowser"), TEXT("Opens a window that previews available slate styles"))
	{}
	
	virtual void DrawWindow(float DeltaTime) override;

private:
	FString SearchString;
	FString PreviewText;
	float SpinBoxValue = 66.7f;
	float SliderValue = 66.7f;
	int32 SelectedComboIndex = INDEX_NONE;
	int32 SelectedListIndex = INDEX_NONE;
};

class FSlateIMTestWindowWidget : public FSlateIMWindowBase
{
public:
	FSlateIMTestWindowWidget(const TCHAR* Command, const TCHAR* CommandHelp)
		: FSlateIMWindowBase(TEXT("SlateIM Test Suite"), FVector2f(700, 900), /* Always On Top */ false, Command, CommandHelp)
	{}
	
	virtual void DrawWindow(float DeltaTime) override;

private:
	FSlateIMTestWidget TestWidget;
};

class FSlateIMTestNomadTabWidget : public FSlateIMNomadTabBase
{
public:
	FSlateIMTestNomadTabWidget(const TCHAR* Command, const TCHAR* CommandHelp)
		: FSlateIMNomadTabBase(TEXT("SlateIM Test Suite"), Command, CommandHelp)
	{}
	
	virtual void DrawContent(float DeltaTime) override;

private:
	FSlateIMTestWidget TestWidget;
};

#if WITH_ENGINE
class FSlateIMTestViewportWidget : public FSlateIMWidgetWithCommandBase
{
public:
	FSlateIMTestViewportWidget(const TCHAR* Command, const TCHAR* CommandHelp)
		: FSlateIMWidgetWithCommandBase(Command, CommandHelp)
	{
		Layout.Anchors = FAnchors(0.5f, 0);
		Layout.Alignment = FVector2f(0.5f, 0);
		Layout.Size = FVector2f(700, 900);
	}
	
	virtual void DrawWidget(float DeltaTime) override;

private:
	FSlateIMTestWidget TestWidget;
	SlateIM::FViewportRootLayout Layout;
};
#endif // WITH_ENGINE
#endif // WITH_SLATEIM_EXAMPLES
