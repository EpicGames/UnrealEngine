// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputCoreTypes.h"

enum class ESlateIMKeyState : uint8
{
	Idle,
	Pressed,
	Held,
	Released,
};

struct FSlateIMInputState
{
	TMap<FKey, ESlateIMKeyState> KeyStateMap;
	TMap<FKey, float> AnalogValueMap;
};
