// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMManager.h"

#include "Containers/SImContextMenuAnchor.h"
#include "Containers/SImStackBox.h"
#include "Containers/SImTab.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/App.h"
#include "Roots/ISlateIMRoot.h"
#include "SlateIMLogging.h"
#include "SlateIM.h"
#include "SlateIMWidgetActivationMetadata.h"

namespace SlateIM
{
	FRootNode::FRootNode(FName InRootName, TSharedPtr<ISlateIMRoot> InRootWidget, bool InRootState)
			: RootContainer(nullptr)
			, RootWidget(InRootWidget)
			, RootName(InRootName)
			, RootState(InRootState)
	{}

	FWidgetHash FRootNode::GetWidgetHash() const
	{
		return WidgetTreeDataHash.Num() > CurrentWidgetIndex ? WidgetTreeDataHash[CurrentWidgetIndex] : FWidgetHash();
	}

	void FRootNode::SetDataHash(FXxHash64 InDataHash)
	{
		UE_LOGF(LogSlateIM, Verbose, "SetDataHash - CurrentWidgetIndex [%d] | WidgetTreeDataHash.Num() [%d]", CurrentWidgetIndex, WidgetTreeDataHash.Num());
		if (WidgetTreeDataHash.Num() <= CurrentWidgetIndex)
		{
			WidgetTreeDataHash.Emplace(FXxHash64(), InDataHash);
			check(WidgetTreeDataHash.Num() - 1 == CurrentWidgetIndex);
		}
		else
		{
			WidgetTreeDataHash[CurrentWidgetIndex].DataHash = InDataHash;
		}
	}

	void FRootNode::SetAlignmentHash(FXxHash64 AlignmentHash)
	{
		UE_LOGF(LogSlateIM, Verbose, "SetAlignmentHash - CurrentWidgetIndex [%d] | WidgetTreeDataHash.Num() [%d]", CurrentWidgetIndex, WidgetTreeDataHash.Num());
		if (WidgetTreeDataHash.Num() <= CurrentWidgetIndex)
		{
			WidgetTreeDataHash.Emplace(AlignmentHash, FXxHash64());
			ensure(WidgetTreeDataHash.Num() - 1 == CurrentWidgetIndex);
		}
		else
		{
			WidgetTreeDataHash[CurrentWidgetIndex].AlignmentHash = AlignmentHash;
		}
	}

	void FRootNode::SetNextToolTip(const FStringView& InNextToolTip)
	{
		if (!InNextToolTip.IsEmpty())
		{
			CurrentToolTip = InNextToolTip;
		}
		else
		{
			CurrentToolTip.Empty();
		}
	}

	void FRootNode::ActivateTab()
	{
		if (!TabsToActivate.IsEmpty())
		{
			for (const FName& TabId : TabsToActivate)
			{
				if (const TWeakPtr<SImTab>* WeakTabPtr = Tabs.Find(TabId))
				{
					if (TSharedPtr<SImTab> Tab = WeakTabPtr->Pin())
					{
						Tab->ActivateInParent(ETabActivationCause::SetDirectly);
					}
				}
			}

			TabsToActivate.Empty();
		}

		Tabs.Empty();
	}

	TUniquePtr<FSlateIMManager> FSlateIMManager::Instance;

	void FSlateIMManager::Initialize()
	{
		if (FSlateApplication::IsInitialized() && !Instance)
		{
			Instance = MakeUnique<FSlateIMManager>();
		}
	}

	FSlateIMManager& FSlateIMManager::Get()
	{
		check(Instance.IsValid());

		return *Instance;
	}

	FSlateIMManager::FSlateIMManager()
#if WITH_SLATEIM_EXAMPLES
			: TestWindowWidget(TEXT("SlateIM.ToggleTestSuiteWindow"), TEXT("Toggles the Slate immediate mode test suite window which demonstrates the capabilities of the Slate immediate mode api"))
			, TestNomadTabWidget(TEXT("SlateIM.ToggleTestSuiteDockableTab"), TEXT("Toggles the Slate immediate mode test suite as a dockable tab which demonstrates the capabilities of the Slate immediate mode api"))
#if WITH_ENGINE
			, TestViewportWidget(TEXT("SlateIM.ToggleTestSuiteViewport"), TEXT("Toggles the Slate immediate mode test suite in the main viewport (PIE or LevelEditor), demonstrating the capabilities of the Slate immediate mode api"))
#endif
#endif
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().OnPostTick().AddRaw(this, &FSlateIMManager::Tick);
			FSlateApplication::Get().OnPreShutdown().AddRaw(this, &FSlateIMManager::OnSlateShutdown);
		}
	}

	FSlateIMManager::~FSlateIMManager()
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().OnPreShutdown().RemoveAll(this);
			FSlateApplication::Get().OnPostTick().RemoveAll(this);
		}
	}

	FRootNode& FSlateIMManager::AddRoot(const FName WindowId, TSharedPtr<ISlateIMRoot> NewRoot)
	{
		return RootMap.Emplace(WindowId, FRootNode(WindowId, NewRoot, true));
	}

	void FSlateIMManager::BeginRoot(FName RootName)
	{
		if (!CanUpdateSlateIM())
		{
			return;
		}
		
		FPlatformMisc::BeginNamedEvent(FColorList::Goldenrod, *RootName.ToString());
		TRACE_CPUPROFILER_EVENT_MANUAL_START(RootName);
		
		checkf(CurrentRoot == nullptr, TEXT("Cannot begin an new SlateIM root while one is already being built. Call SlateIM::EndRoot() to end current root and begin a new one"));

		check(MemMark == nullptr);

		MemMark = MakeUnique<FMemMark>(FMemStack::Get());

		CurrentRoot = &RootMap.FindChecked(RootName);
		CurrentRoot->LastAccessTime = FApp::GetCurrentTime();

		CurrentRoot->bCurrentEnabledState = true;
		CurrentRoot->bActivatedThisFrame = true;
		CurrentRoot->CurrentContainerStack.Reset();
		CurrentRoot->CurrentMenuAnchorStack.Reset();

		CurrentRoot->CurrentWidgetIndex = 0;

		if (CurrentRoot->RootContainer.IsValid())
		{
			CurrentRoot->CurrentContainerStack.Push(CurrentRoot->RootContainer);
		}
	}

	void FSlateIMManager::EndRoot()
	{
		if (!CanUpdateSlateIM())
		{
			return;
		}
		
		checkf(CurrentRoot != nullptr, TEXT("Called SlateIM::EndRoot() without an active root"));

		if (CurrentRoot->CurrentContainerStack.Num() > 0)
		{
			// TODO - Is this an error caused by a missing Pop?
			RemoveUnusedChildren(CurrentRoot->CurrentContainerStack.Top());
		}

		CurrentRoot->CurrentContainerStack.Reset();
		CurrentRoot->CurrentMenuAnchorStack.Reset();

		const int32 FirstUnusedIndex = CurrentRoot->CurrentWidgetIndex + 1;
		if (FirstUnusedIndex < CurrentRoot->WidgetTreeDataHash.Num())
		{
			CurrentRoot->WidgetTreeDataHash.RemoveAt(FirstUnusedIndex, CurrentRoot->WidgetTreeDataHash.Num() - FirstUnusedIndex, EAllowShrinking::No);
		}

		CurrentRoot->ActivateTab();

		CurrentRoot->CurrentWidgetIndex = 0;
		CurrentRoot = nullptr;

		MemMark.Reset();

		TRACE_CPUPROFILER_EVENT_MANUAL_END();
		FPlatformMisc::EndNamedEvent();
	}

	void FSlateIMManager::PushContainer(FContainerNode&& Node)
	{
		// Container widget should exist in the parent
		CurrentRoot->CurrentContainerStack.Push(Node);
	}

	const FContainerNode* FSlateIMManager::GetCurrentContainerNode() const
	{
		return CurrentRoot->CurrentContainerStack.IsEmpty() ? nullptr : &CurrentRoot->CurrentContainerStack.Top();
	}

	void FSlateIMManager::PushMenuRoot(TSharedPtr<SImContextMenuAnchor>& MenuRoot)
	{
		MenuRoot->Begin();
		CurrentRoot->CurrentMenuAnchorStack.Push(MenuRoot);
	}

	void FSlateIMManager::PopMenuRoot()
	{
		TSharedPtr<SImContextMenuAnchor> MenuRoot = CurrentRoot->CurrentMenuAnchorStack.Pop(EAllowShrinking::No);
		MenuRoot->End();
	}

	TSharedPtr<SImContextMenuAnchor> FSlateIMManager::GetCurrentMenuRoot() const
	{
		return CurrentRoot->CurrentMenuAnchorStack.Num() ? CurrentRoot->CurrentMenuAnchorStack.Top() : nullptr;
	}

	TSharedPtr<SWidget> FSlateIMManager::BeginCustomWidget(TSharedPtr<SWidget> ExpectedWidget)
	{
		bBuildingImmediateModeWidgets = true;

		EnsureCurrentContainerNode();
		AdvanceToNextWidget();

		if (GetCurrentChild().GetWidget() == ExpectedWidget)
		{
			return ExpectedWidget;
		}
			
		return nullptr;
	}

	void FSlateIMManager::EndWidget(bool bResetAlignmentData)
	{
		bBuildingImmediateModeWidgets = false;

		if (bResetAlignmentData)
		{
			ResetAlignmentData();
		}
	}

	void FSlateIMManager::EnsureCurrentContainerNode()
	{
		check(CurrentRoot != nullptr);
		
		// Add new vertical box (the default layout) as a container inside the current root. The current root always has one child
		if (CurrentRoot->CurrentContainerStack.Num() == 0 && !CurrentRoot->RootContainer.IsValid())
		{
			UE_LOGF(LogSlateIM, Log, "Adding default root container");
			TSharedRef<SImStackBox> VerticalBox = SNew(SImStackBox).Orientation(Orient_Vertical);

			const FSlateIMSlotData AlignmentData(FMargin(0), HAlign_Fill, VAlign_Fill, false, 0, 0, 0, 0);

			CurrentRoot->RootWidget->UpdateChild(VerticalBox, AlignmentData);

			CurrentRoot->SetAlignmentHash(AlignmentData.Hash);

			CurrentRoot->RootContainer = FContainerNode(VerticalBox);

			CurrentRoot->CurrentContainerStack.Push(CurrentRoot->RootContainer);
		}
	}
	
	FSlateIMChild FSlateIMManager::GetCurrentChild() const
	{
		check(CurrentRoot->CurrentContainerStack.Num());

		const FContainerNode& CurrentContainer = CurrentRoot->CurrentContainerStack.Top();
		
		return ensure(CurrentContainer.Widget) ? CurrentContainer.Widget->GetChild(CurrentContainer.LastUsedChildIndex) : nullptr;
	}

	TSharedPtr<SWidget> FSlateIMManager::GetCurrentChildAsWidget() const
	{
		FSlateIMChild CurrentChild = GetCurrentChild();
		TSharedPtr<SWidget> ChildWidget = CurrentChild.GetWidget();

		if (!ChildWidget)
		{
			if (TSharedPtr<ISlateIMChild> Child = CurrentChild.GetChild())
			{
				ChildWidget = Child->GetAsWidget();
			}
		}

		return ChildWidget;
	}

	const FRootNode& FSlateIMManager::GetCurrentRoot() const
	{
		check(CurrentRoot != nullptr);
		return *CurrentRoot; 
	}

	FRootNode& FSlateIMManager::GetMutableCurrentRoot()
	{ 
		check(CurrentRoot != nullptr);
		return *CurrentRoot;
	}

	void FSlateIMManager::UpdateCurrentChild(FSlateIMChild Child, const FSlateIMSlotData& AlignmentData)
	{
		FContainerNode& CurrentContainer = CurrentRoot->CurrentContainerStack.Top();
		CurrentRoot->SetAlignmentHash(AlignmentData.Hash);

		check(CurrentContainer.Widget);
		UE_LOGF(LogSlateIM, Verbose, "Updating Container: [%ls] Child: [%d]", *CurrentContainer.Widget->GetDebugName(), CurrentContainer.LastUsedChildIndex);
		CurrentContainer.Widget->UpdateChild(Child, CurrentContainer.LastUsedChildIndex, AlignmentData);
	}

	void FSlateIMManager::AdvanceToNextWidget()
	{
		++CurrentRoot->CurrentWidgetIndex;
		++CurrentRoot->CurrentContainerStack.Top().LastUsedChildIndex;
	}

	void FSlateIMManager::ActivateWidget(const TSharedPtr<FSlateIMWidgetActivationMetadata>& ActivationData)
	{
		if (ActivationData.IsValid())
		{
			// Setting values while building immediate mode ui causes some widgets to activate
			// This is undesirable as it can cause feedback loops
			if (!bBuildingImmediateModeWidgets)
			{
				FWidgetActivation WidgetActivation = FWidgetActivation{ ActivationData->RootName, ActivationData->ContainerIndex, ActivationData->WidgetIndex };
				UE_LOGF(LogSlateIM, Verbose, "ActivateWidget | Root: [%ls] | Container [%d] | Widget [%d]", *WidgetActivation.RootName.ToString(), WidgetActivation.ContainerIndex, WidgetActivation.WidgetIndex);
				ActivatedWidgets.Add(MoveTemp(WidgetActivation));
			}
		}
	}

	bool FSlateIMManager::IsWidgetActivatedThisFrame(const TSharedPtr<FSlateIMWidgetActivationMetadata>& ActivationData) const
	{
		if (ActivationData.IsValid())
		{
			const FWidgetActivation WidgetActivation = FWidgetActivation{ ActivationData->RootName, ActivationData->ContainerIndex, ActivationData->WidgetIndex };
			return ActivatedWidgets.Contains(WidgetActivation);
		}

		return false;
	}

	void FSlateIMManager::ResetAlignmentData()
	{
		NextPadding.Reset();
		NextHAlign.Reset();
		NextVAlign.Reset();
		NextAutoSize.Reset();
		NextMinWidth.Reset();
		NextMinHeight.Reset();
		NextMaxWidth.Reset();
		NextMaxHeight.Reset();
	}

	void FSlateIMManager::OnSlateIMModalOpened()
	{
		ensure(!bIsSlateIMModalOpen);
		bIsSlateIMModalOpen = true;
	}

	void FSlateIMManager::OnSlateIMModalClosed()
	{
		bIsSlateIMModalOpen = false;
	}

	bool FSlateIMManager::CanUpdateSlateIM() const
	{
		return !bIsSlateIMModalOpen && !UE::IsSavingPackage(nullptr) && FSlateApplication::IsInitialized() && FSlateApplication::Get().GetNumExternalModalWindowsActive() == 0;
	}

	void FSlateIMManager::ValidateRootName(FName RootName) const
	{
		checkf(!RootMap.Contains(RootName), TEXT("SlateIM: Root Name %s is not unique. All roots of Slate IM hierarchies must have a unique name"), *RootName.ToString());
	}

	void FSlateIMManager::Tick(float DeltaTime)
	{
		SCOPED_NAMED_EVENT_TEXT("FSlateIMManager::Tick", FColorList::Goldenrod);
		if (!CanUpdateSlateIM())
		{
			return;
		}

		checkf(CurrentRoot == nullptr, TEXT("There's an active SlateIM Root, are you missing a call to SlateIM::EndRoot()?"));

		ActivatedWidgets.Reset();

		for (auto It = RootMap.CreateIterator(); It; ++It)
		{
			if (It.Value().bActivatedThisFrame == false)
			{
				It.RemoveCurrent();
			}
			else
			{
				// Reset for next frame
				It.Value().bActivatedThisFrame = false;
			}
		}
	}

	void FSlateIMManager::OnSlateShutdown()
	{
		ActivatedWidgets.Empty();
		RootMap.Empty();
	}

	void FSlateIMManager::RemoveUnusedChildren(FContainerNode& Container)
	{
		if (Container.Widget->GetNumChildren() > Container.LastUsedChildIndex + 1)
		{
			TSharedPtr<ISlateIMContainer> ContainerWidget = Container.Widget;
			ContainerWidget->RemoveUnusedChildren(Container.LastUsedChildIndex);
		}
	}

	void FSlateIMManager::PopContainer_Internal()
	{
		RemoveUnusedChildren(CurrentRoot->CurrentContainerStack.Top());
		CurrentRoot->CurrentContainerStack.Pop(EAllowShrinking::No);
	}
}
