// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ISlateIMContainer.h"
#include "Containers/Union.h"
#include "Hash/xxhash.h"
#include "Layout/Margin.h"
#include "Roots/ISlateIMRoot.h"
#include "SlateIMLogging.h"
#include "SlateIMSlotData.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SWidget.h"

#if WITH_SLATEIM_EXAMPLES
#include "SlateIMExamples.h"
#endif

struct FSlateIMWidgetActivationMetadata;
class FMemMark;
class ISlateIMChild;
class ISlateIMRoot;
class ISlateIMContainer;
class SImContextMenuAnchor;
class SImTab;
class SWidget;

struct FSlateIMSlotData;

namespace SlateIM
{
	struct FContainerNode
	{
		FContainerNode(TSharedPtr<ISlateIMContainer> InWidget)
			: Widget(InWidget)
		{}

		bool IsValid() const { return Widget.IsValid(); }

		/** Current State */
		TSharedPtr<ISlateIMContainer> Widget;
		int32 LastUsedChildIndex = INDEX_NONE;
	};
	
	struct FWidgetHash
	{
		FXxHash64 AlignmentHash;
		FXxHash64 DataHash;

		FWidgetHash()
		{}

		FWidgetHash(FXxHash64 InAlignmentHash, FXxHash64 InDataHash)
			: AlignmentHash(InAlignmentHash)
			, DataHash(InDataHash)
		{}

		bool IsValid() const
		{
			return !AlignmentHash.IsZero() && !DataHash.IsZero();
		}
	};
	
	struct FRootNode
	{
		FRootNode(FName InRootName, TSharedPtr<ISlateIMRoot> InRootWidget, bool InRootState);

		FWidgetHash GetWidgetHash() const;

		void SetDataHash(FXxHash64 InDataHash);

		void SetAlignmentHash(FXxHash64 AlignmentHash);

		void SetDisabledState() { bCurrentEnabledState = false; }
		void SetEnabledState() { bCurrentEnabledState = true; }

		void SetNextToolTip(const FStringView& InNextToolTip);

		void ActivateTab();

		FContainerNode RootContainer;

		TArray<FContainerNode> CurrentContainerStack;
		TArray<TSharedPtr<SImContextMenuAnchor>> CurrentMenuAnchorStack;

		TArray<FWidgetHash> WidgetTreeDataHash;

		TSharedPtr<ISlateIMRoot> RootWidget;

		// Store a list of tabs so that when ActivateTab is called at the end of layout, any registered tab can be activated.
		TMap<FName, TWeakPtr<SImTab>> Tabs;
		TSet<FName> TabsToActivate;

		FString CurrentToolTip;
		uint64 DataHash = 0;
		FName RootName = NAME_None;
		int32 CurrentWidgetIndex = 0;
		double LastAccessTime = 0;
		bool RootState = false;
		bool bCurrentEnabledState = true;
		bool bActivatedThisFrame = false;
	};

	struct FWidgetActivation
	{
		FName RootName = NAME_None;
		int32 ContainerIndex = INDEX_NONE;
		int32 WidgetIndex = INDEX_NONE;

		bool operator==(const FWidgetActivation& Other) const
		{
			return RootName == Other.RootName && ContainerIndex == Other.ContainerIndex && WidgetIndex == Other.WidgetIndex;
		}
	};

	struct FSlateIMAlignmentState
	{
		TOptional<FMargin> Padding;
		TOptional<EHorizontalAlignment> HAlign;
		TOptional<EVerticalAlignment> VAlign;
		TOptional<bool> AutoSize;
		TOptional<float> MinWidth;
		TOptional<float> MinHeight;
		TOptional<float> MaxWidth;
		TOptional<float> MaxHeight;
	};
	
	class FSlateIMManager
	{
	public:
		static void Initialize();
		static FSlateIMManager& Get();

		FSlateIMManager();
		~FSlateIMManager();

		template<typename RootType>
		FRootNode* FindRoot(const FName WindowId);
		FRootNode& AddRoot(const FName WindowId, TSharedPtr<ISlateIMRoot> NewRoot);

		void BeginRoot(FName RootName);
		void EndRoot();
		
		void PushContainer(FContainerNode&& Node);
		template<typename ContainerType>
		void PopContainer();
		template<typename ContainerType>
		TSharedPtr<ContainerType> GetCurrentContainer() const;
		template<typename ContainerType>
		TSharedPtr<ContainerType> FindMostRecentContainer() const;
		const FContainerNode* GetCurrentContainerNode() const;
		
		void PushMenuRoot(TSharedPtr<SImContextMenuAnchor>& MenuRoot);
		void PopMenuRoot();
		TSharedPtr<SImContextMenuAnchor> GetCurrentMenuRoot() const;

		template<typename WidgetType>
		TSharedPtr<WidgetType> BeginIMWidget();
		TSharedPtr<SWidget> BeginCustomWidget(TSharedPtr<SWidget> ExpectedWidget);
		void EndWidget(bool bResetAlignmentData);

		void EnsureCurrentContainerNode();

		/** Gets the widget which should be at the current spot in the hierarchy or null if it doesn't exist */
		FSlateIMChild GetCurrentChild() const;
		TSharedPtr<SWidget> GetCurrentChildAsWidget() const;
		const FRootNode& GetCurrentRoot() const;
		FRootNode& GetMutableCurrentRoot();
		bool HasCurrentRoot() const { return !!CurrentRoot; }

		void UpdateCurrentChild(FSlateIMChild Child, const FSlateIMSlotData& AlignmentData);
		void AdvanceToNextWidget();

		void ActivateWidget(const TSharedPtr<FSlateIMWidgetActivationMetadata>& ActivationData);
		bool IsWidgetActivatedThisFrame(const TSharedPtr<FSlateIMWidgetActivationMetadata>& ActivationData) const;

		FSlateIMSlotData GetCurrentAlignmentData(
			const FMargin& DefaultPadding,
			const EHorizontalAlignment DefaultHAlign,
			const EVerticalAlignment DefaultVAlign,
			const bool bDefaultAutoSize,
			const float DefaultMinWidth,
			const float DefaultMinHeight,
			const float DefaultMaxWidth,
			const float DefaultMaxHeight) const
		{
			return FSlateIMSlotData(
				NextPadding.Get(DefaultPadding),
				NextHAlign.Get(DefaultHAlign),
				NextVAlign.Get(DefaultVAlign),
				NextAutoSize.Get(bDefaultAutoSize),
				NextMinWidth.Get(DefaultMinWidth),
				NextMinHeight.Get(DefaultMinHeight),
				NextMaxWidth.Get(DefaultMaxWidth),
				NextMaxHeight.Get(DefaultMaxHeight)
			);
		}
		void ResetAlignmentData();

		FSlateIMAlignmentState SaveAlignmentState() const
		{
			return {
				NextPadding,
				NextHAlign,
				NextVAlign,
				NextAutoSize,
				NextMinWidth,
				NextMinHeight,
				NextMaxWidth,
				NextMaxHeight
			};
		}

		void RestoreAlignmentState(const FSlateIMAlignmentState& AlignmentState)
		{
			NextPadding = AlignmentState.Padding;
			NextHAlign = AlignmentState.HAlign;
			NextVAlign = AlignmentState.VAlign;
			NextAutoSize = AlignmentState.AutoSize;
			NextMinWidth = AlignmentState.MinWidth;
			NextMinHeight = AlignmentState.MinHeight;
			NextMaxWidth = AlignmentState.MaxWidth;
			NextMaxHeight = AlignmentState.MaxHeight;
		}

		void OnSlateIMModalOpened();
		void OnSlateIMModalClosed();
		bool CanUpdateSlateIM() const;
		
		TOptional<FMargin> NextPadding;
		TOptional<EHorizontalAlignment> NextHAlign;
		TOptional<EVerticalAlignment> NextVAlign;
		TOptional<bool> NextAutoSize;
		TOptional<float> NextMinWidth;
		TOptional<float> NextMinHeight;
		TOptional<float> NextMaxWidth;
		TOptional<float> NextMaxHeight;

	private:
		void ValidateRootName(FName RootName) const;

		void Tick(float DeltaTime);

		void OnSlateShutdown();

		void RemoveUnusedChildren(FContainerNode& Container);

		void PopContainer_Internal();

		static TUniquePtr<FSlateIMManager> Instance;
		
		TMap<FName, FRootNode> RootMap;
		TArray<FWidgetActivation, TInlineAllocator<4>> ActivatedWidgets;
		FRootNode* CurrentRoot = nullptr;

		TUniquePtr<FMemMark> MemMark;

		bool bBuildingImmediateModeWidgets = false;
		bool bIsSlateIMModalOpen = false;

#if WITH_SLATEIM_EXAMPLES
		FSlateStyleBrowser SlateStyleBrowser;
		FSlateIMTestWindowWidget TestWindowWidget;
		FSlateIMTestNomadTabWidget TestNomadTabWidget;
#if WITH_ENGINE
		FSlateIMTestViewportWidget TestViewportWidget;
#endif // WITH_ENGINE
#endif // WITH_SLATEIM_EXAMPLES
	};

	template <typename RootType>
	FRootNode* FSlateIMManager::FindRoot(const FName WindowId)
	{
		static_assert(std::is_base_of_v<ISlateIMRoot, RootType>, "RootType must be a subclass of ISlateIMRoot.");
		FRootNode* RootNode = RootMap.Find(WindowId);

		if (RootNode && RootNode->RootWidget && RootNode->RootWidget->IsA<RootType>())
		{
			return RootNode;
		}

		return nullptr;
	}

	template <typename ContainerType>
	void FSlateIMManager::PopContainer()
	{
		checkf(GetCurrentContainer<ContainerType>().IsValid(), TEXT("The container being popped is not the expected type. Are your Begin and End function calls mismatched?"));
		PopContainer_Internal();
	}

	template <typename ContainerType>
	TSharedPtr<ContainerType> FSlateIMManager::GetCurrentContainer() const
	{
		static_assert(std::is_base_of_v<ISlateIMContainer, ContainerType>, "ContainerType must derive from ISlateIMContainer");
		TSharedPtr<ISlateIMContainer> Container = CurrentRoot->CurrentContainerStack.IsEmpty() ? nullptr : CurrentRoot->CurrentContainerStack.Top().Widget;
		if (Container && Container->IsA<ContainerType>())
		{
			return StaticCastSharedPtr<ContainerType>(Container);
		}

		return nullptr;
	}

	template <typename ContainerType>
	TSharedPtr<ContainerType> FSlateIMManager::FindMostRecentContainer() const
	{
		static_assert(std::is_base_of_v<ISlateIMContainer, ContainerType>, "ContainerType must derive from ISlateIMContainer");

		for (int32 i = CurrentRoot->CurrentContainerStack.Num() - 1; i >= 0; --i)
		{
			TSharedPtr<ISlateIMContainer> Container = CurrentRoot->CurrentContainerStack[i].Widget;
			if (Container && Container->IsA<ContainerType>())
			{
				return StaticCastSharedPtr<ContainerType>(Container);
			}
		}

		return nullptr;
	}

	template <typename WidgetType>
	TSharedPtr<WidgetType> FSlateIMManager::BeginIMWidget()
	{
		bBuildingImmediateModeWidgets = true;

		EnsureCurrentContainerNode();
		AdvanceToNextWidget();

		if constexpr (std::is_base_of_v<SWidget, WidgetType>)
		{
			return GetCurrentChild().GetWidget<WidgetType>();
		}
		else if constexpr (std::is_base_of_v<ISlateIMChild, WidgetType>)
		{
			return GetCurrentChild().GetChild<WidgetType>();
		}
		else
		{
			checkNoEntry();
			return nullptr;
		}
	}
}
