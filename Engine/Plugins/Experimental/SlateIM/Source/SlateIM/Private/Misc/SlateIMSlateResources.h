// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Styling/SlateBrush.h"
#include "Types/ISlateMetaData.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

namespace SlateIM
{
	template<typename DataType>
	struct FSlateIMDataStore : ISlateMetaData
	{
		SLATE_METADATA_TYPE(FSlateIMDataStore, ISlateMetaData);

		FSlateIMDataStore(DataType&& InData)
			: Data(MoveTemp(InData))
		{
		}

		FSlateIMDataStore(const DataType& InData)
			: Data(InData)
		{
		}

		DataType Data;
	};

	/**
	 * Object Resource
	 */
	struct FPinnedObjectResource
	{
		TStrongObjectPtr<UObject> PinnedResource;
	};

	using FUObjectObjectResource = FSlateIMDataStore<FPinnedObjectResource>;

	struct FSlateIMPinnedObjectResourceDataStore : FSlateIMDataStore<FPinnedObjectResource>
	{
		using Super = FSlateIMDataStore<FPinnedObjectResource>;

		FSlateIMPinnedObjectResourceDataStore(FPinnedObjectResource&& InData)
			: Super(MoveTemp(InData))
		{
		}

		FSlateIMPinnedObjectResourceDataStore(const FPinnedObjectResource& InData)
			: Super(InData)
		{
		}

		static const FName& GetTypeId() { static FName Type(TEXT("FSlateIMPinnedObjectResourceDataStore")); return Type; }
		virtual bool IsOfTypeImpl(const FName& Type) const override { return GetTypeId() == Type || Super::IsOfTypeImpl(Type); }
	};

	/**
	 * Image Resource
	 */
	struct FPinnedImageResource
	{
		FSlateBrush Brush;
		TStrongObjectPtr<UObject> PinnedResource;
	};

	using FUObjectImageResource = FSlateIMDataStore<FPinnedImageResource>;

	struct FSlateIMPinnedImageResourceDataStore : FSlateIMDataStore<FPinnedImageResource>
	{
		using Super = FSlateIMDataStore<FPinnedImageResource>;

		FSlateIMPinnedImageResourceDataStore(FPinnedImageResource&& InData)
			: Super(MoveTemp(InData))
		{
		}

		FSlateIMPinnedImageResourceDataStore(const FPinnedImageResource& InData)
			: Super(InData)
		{
		}

		static const FName& GetTypeId() { static FName Type(TEXT("FSlateIMPinnedImageResourceDataStore")); return Type; }
		virtual bool IsOfTypeImpl(const FName& Type) const override { return GetTypeId() == Type || Super::IsOfTypeImpl(Type); }
	};
}
