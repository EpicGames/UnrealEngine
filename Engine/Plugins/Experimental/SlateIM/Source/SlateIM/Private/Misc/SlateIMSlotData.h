// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "Hash/xxhash.h"

namespace SlateIM
{
	namespace Defaults
	{
		const FMargin Padding(3.0f, 3.0f, 3.0f, 3.0f);
		constexpr EHorizontalAlignment HAlign = HAlign_Left;
		constexpr EVerticalAlignment VAlign = VAlign_Fill;
		constexpr bool bAutoSize = true;
		constexpr float MinWidth = 0.0f;
		constexpr float MinHeight = 0.0f;
		constexpr float MaxWidth = 0.0f;
		constexpr float MaxHeight = 0.0f;
		constexpr float InputWidgetWidth = 200.f;
	}
}

struct FSlateIMSlotData
{
	FSlateIMSlotData(const FMargin& InPadding, const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign, bool bInAutoSize, float InMinWidth, float InMinHeight, float InMaxWidth, float InMaxHeight)
		: Padding(InPadding)
		, HorizontalAlignment(InHAlign)
		, VerticalAlignment(InVAlign)
		, bAutoSize(bInAutoSize)
		, MinWidth(InMinWidth)
		, MinHeight(InMinHeight)
		, MaxWidth(InMaxWidth)
		, MaxHeight(InMaxHeight)
		, Hash(GetAlignmentHash())
	{
	}

	const FMargin Padding = SlateIM::Defaults::Padding;
	const EHorizontalAlignment HorizontalAlignment = SlateIM::Defaults::HAlign;
	const EVerticalAlignment VerticalAlignment = SlateIM::Defaults::VAlign;
	const bool bAutoSize = SlateIM::Defaults::bAutoSize;
	const float MinWidth = SlateIM::Defaults::MinWidth;
	const float MinHeight = SlateIM::Defaults::MinHeight;
	const float MaxWidth = SlateIM::Defaults::MaxWidth;
	const float MaxHeight = SlateIM::Defaults::MaxHeight;
	const FXxHash64 Hash;

private:
	FXxHash64 GetAlignmentHash() const
	{
		FXxHash64Builder HashBuilder;

		HashBuilder.Update(&Padding, sizeof(FMargin));

		const char AlignmentVals[] = { (char)HorizontalAlignment, (char)VerticalAlignment };

		HashBuilder.Update(AlignmentVals, sizeof(AlignmentVals));
		HashBuilder.Update(&bAutoSize, sizeof(bAutoSize));
		HashBuilder.Update(&MinWidth, sizeof(MinWidth));
		HashBuilder.Update(&MinHeight, sizeof(MinHeight));
		HashBuilder.Update(&MaxWidth, sizeof(MaxWidth));
		HashBuilder.Update(&MaxHeight, sizeof(MaxHeight));

		return HashBuilder.Finalize();
	}
};