// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

class ISlateIMTypeChecking
{
public:
	template<class TType>
	bool IsA() const
	{
		return IsAImpl(TType::GetTypeId());
	}

	virtual ~ISlateIMTypeChecking() = default;

protected:
	virtual bool IsAImpl(const FName&) const
	{
		return false;
	}
};

#define SLATE_IM_TYPE_DATA(TYPE, BASE) \
	public: \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	private: \
	virtual bool IsAImpl(const FName& Type) const override { return GetTypeId() == Type || BASE::IsAImpl(Type); }
