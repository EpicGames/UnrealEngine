// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/ISlateMetaData.h"

struct FSlateIMWidgetActivationMetadata : ISlateMetaData
{
	SLATE_METADATA_TYPE(FSlateIMWidgetActivationMetadata, ISlateMetaData);

	FSlateIMWidgetActivationMetadata(const FName& RootName, int32 ContainerIndex, int32 WidgetIndex)
		: RootName(RootName)
		, ContainerIndex(ContainerIndex)
		, WidgetIndex(WidgetIndex)	
	{}
	
	FName RootName = NAME_None;
	int32 ContainerIndex = INDEX_NONE;
	int32 WidgetIndex = INDEX_NONE;
};