// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateIMManager.h"
#include "SlateIMWidgetActivationMetadata.h"

namespace SlateIM
{
	template<typename WidgetType, bool CanHaveMetadata = TIsDerivedFrom<WidgetType, SWidget>::Value>
	struct FWidgetScope
	{
		FWidgetScope(const FMargin DefaultPadding = Defaults::Padding,
			const EHorizontalAlignment DefaultHAlign = Defaults::HAlign,
			const EVerticalAlignment DefaultVAlign = Defaults::VAlign,
			const bool bDefaultAutoSize = Defaults::bAutoSize,
			const float DefaultMinWidth = Defaults::MinWidth,
			const float DefaultMinHeight = Defaults::MinHeight,
			const float DefaultMaxWidth = Defaults::MaxWidth,
			const float DefaultMaxHeight = Defaults::MaxHeight)
			: Widget(FSlateIMManager::Get().BeginIMWidget<WidgetType>())
			, AlignmentData(FSlateIMManager::Get().GetCurrentAlignmentData(DefaultPadding, DefaultHAlign, DefaultVAlign, bDefaultAutoSize, DefaultMinWidth, DefaultMinHeight, DefaultMaxWidth, DefaultMaxHeight))
			, DataHash()
			, bIsSlotDirty(false)
		{
			HashStringView(FSlateIMManager::Get().GetCurrentRoot().CurrentToolTip);

			if constexpr (CanHaveMetadata)
			{
				if (Widget.IsValid())
				{
					ActivationMetadata = Widget->template GetMetaData<FSlateIMWidgetActivationMetadata>();
				}
			}
		}

		FWidgetScope(const bool bAutoSize)
			: FWidgetScope(Defaults::Padding, Defaults::HAlign, Defaults::VAlign, bAutoSize)
		{
		}

		FWidgetScope(const TSharedPtr<SWidget> ExpectedWidget,
			const FMargin DefaultPadding = Defaults::Padding,
			const EHorizontalAlignment DefaultHAlign = Defaults::HAlign,
			const EVerticalAlignment DefaultVAlign = Defaults::VAlign,
			const bool bDefaultAutoSize = Defaults::bAutoSize,
			const float DefaultMinWidth = Defaults::MinWidth,
			const float DefaultMinHeight = Defaults::MinHeight,
			const float DefaultMaxWidth = Defaults::MaxWidth,
			const float DefaultMaxHeight = Defaults::MaxHeight)
			: Widget(FSlateIMManager::Get().BeginCustomWidget(ExpectedWidget))
			, AlignmentData(FSlateIMManager::Get().GetCurrentAlignmentData(DefaultPadding, DefaultHAlign, DefaultVAlign, bDefaultAutoSize, DefaultMinWidth, DefaultMinHeight, DefaultMaxWidth, DefaultMaxHeight))
			, DataHash()
			, bIsSlotDirty(false)
		{
			HashStringView(FSlateIMManager::Get().GetCurrentRoot().CurrentToolTip);

			if constexpr (CanHaveMetadata)
			{
				if (Widget.IsValid())
				{
					ActivationMetadata = Widget->template GetMetaData<FSlateIMWidgetActivationMetadata>();
				}
			}
		}

		~FWidgetScope()
		{
			if constexpr (CanHaveMetadata)
			{
				// Update the activation metadata to match the current frame's values
				if (ActivationMetadata.IsValid())
				{
					const FRootNode& Root = FSlateIMManager::Get().GetCurrentRoot();
					ActivationMetadata->RootName = Root.RootName;
					ActivationMetadata->ContainerIndex = Root.CurrentContainerStack.Num() - 1;
					ActivationMetadata->WidgetIndex = Root.CurrentWidgetIndex;
				}
			}

			FRootNode& CurrentRoot = FSlateIMManager::Get().GetMutableCurrentRoot();

			FWidgetHash CurrentHash = CurrentRoot.GetWidgetHash();

			bIsSlotDirty |= (CurrentHash.AlignmentHash != AlignmentData.Hash);

			if (bIsSlotDirty)
			{
				FSlateIMManager::Get().UpdateCurrentChild(Widget ? FSlateIMChild(Widget.ToSharedRef()) : FSlateIMChild(), AlignmentData);
			}

			if constexpr (std::is_base_of_v<SWidget, WidgetType>)
			{
				if (Widget)
				{
					Widget->SetEnabled(CurrentRoot.bCurrentEnabledState);

					DataHash = HashBuilder.Finalize();

					if (CurrentHash.DataHash != DataHash)
					{
						Widget->SetToolTipText(FText::FromStringView(CurrentRoot.CurrentToolTip));
						CurrentRoot.SetDataHash(DataHash);
					}

					CurrentRoot.SetNextToolTip(FStringView());
				}
				
				constexpr bool bResetAlignmentData = true;
				FSlateIMManager::Get().EndWidget(bResetAlignmentData);
			}
			else
			{
				DataHash = HashBuilder.Finalize();

				if (CurrentHash.DataHash != DataHash)
				{
					CurrentRoot.SetDataHash(DataHash);
				}

				// This "widget" is virtual, let the next widget use the current alignment data
				constexpr bool bResetAlignmentData = false;
				FSlateIMManager::Get().EndWidget(bResetAlignmentData);
			}
		}

		TSharedPtr<WidgetType> GetWidget() { return Widget; }

		void UpdateWidget(TSharedPtr<WidgetType> NewWidget)
		{
			Widget = NewWidget;
			bIsSlotDirty = true;
			
			if constexpr (CanHaveMetadata)
			{
				if (ActivationMetadata.IsValid() && Widget.IsValid() && !Widget->template GetMetaData<FSlateIMWidgetActivationMetadata>().IsValid())
				{
					Widget->AddMetadata(ActivationMetadata.ToSharedRef());
				}
			}
		}

		TSharedPtr<WidgetType> GetOrCreate(const TFunction<TSharedPtr<WidgetType>(void)>& WidgetCreationFunction)
		{
			if (!Widget)
			{
				TSharedPtr<WidgetType> NewWidget = WidgetCreationFunction();
				UpdateWidget(NewWidget);
			}

			HashData(PTRINT(Widget.Get()));
			return Widget;
		}

		template<typename T>
		void HashData(const T& Data)
		{
			HashBuilder.Update(reinterpret_cast<const char*>(&Data), sizeof(T));
		}

		void HashStringView(const FStringView& String)
		{
			HashBuilder.Update(reinterpret_cast<const char*>(String.GetData()), String.Len() * sizeof(TCHAR));
		}

		bool IsDataHashDirty() const
		{
			DataHash = HashBuilder.Finalize();
			return FSlateIMManager::Get().GetCurrentRoot().GetWidgetHash().DataHash != DataHash;
		}

		bool IsActivatedThisFrame() const
		{
			return FSlateIMManager::Get().IsWidgetActivatedThisFrame(ActivationMetadata);
		}

		TSharedPtr<FSlateIMWidgetActivationMetadata> GetOrCreateActivationMetadata()
		{
			if constexpr (CanHaveMetadata)
			{
				if (!ActivationMetadata.IsValid())
				{
					if (Widget.IsValid())
					{
						ActivationMetadata = Widget->template GetMetaData<FSlateIMWidgetActivationMetadata>();
					}
					
					if (!ActivationMetadata.IsValid())
					{
						const FRootNode& Root = FSlateIMManager::Get().GetCurrentRoot();
						const int32 ContainerIndex = Root.CurrentContainerStack.Num() - 1;
						const int32 WidgetIndex = Root.CurrentWidgetIndex;
						ActivationMetadata = MakeShared<FSlateIMWidgetActivationMetadata>(Root.RootName, ContainerIndex, WidgetIndex);
					}
					
					if (Widget.IsValid())
					{
						Widget->AddMetadata(ActivationMetadata.ToSharedRef());
					}
				}
			}
			
			return ActivationMetadata;
		}

	private:
		TSharedPtr<WidgetType> Widget;
		TSharedPtr<FSlateIMWidgetActivationMetadata> ActivationMetadata;
		FSlateIMSlotData AlignmentData;
		FXxHash64Builder HashBuilder;
		mutable FXxHash64 DataHash;
		bool bIsSlotDirty = false;
	};
}
