// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Math/MathFwd.h"
#include "Misc/SlateIMTypeChecking.h"
#include "Templates/SharedPointer.h"

struct FSlateIMInputState;
struct FSlateIMSlotData;
class SDockTab;
class SWidget;

class ISlateIMRoot : public ISlateIMTypeChecking
{
public:
	virtual void UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData) = 0;
	virtual bool IsVisible() const = 0;

	virtual FSlateIMInputState& GetInputState() = 0;

	virtual TSharedRef<SDockTab> GetRootTab() const = 0;
};
