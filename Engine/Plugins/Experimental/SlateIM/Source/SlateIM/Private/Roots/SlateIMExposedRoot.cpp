// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMExposedRoot.h"

#include "Misc/SlateIMSlotData.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SImWrapper.h"


FSlateIMExposedRoot::FSlateIMExposedRoot()
	: ExposedWidget(SNew(SImWrapper))
	, RootTab(SNew(SDockTab))
{
}

FSlateIMExposedRoot::~FSlateIMExposedRoot()
{
	ExposedWidget->SetContent(SNullWidget::NullWidget);
}

void FSlateIMExposedRoot::UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData)
{
	ExposedWidget->SetContent(Child);
	ExposedWidget->SetPadding(AlignmentData.Padding);
	ExposedWidget->SetHAlign(AlignmentData.HorizontalAlignment);
	ExposedWidget->SetVAlign(AlignmentData.VerticalAlignment);
}

FSlateIMInputState& FSlateIMExposedRoot::GetInputState()
{
	return ExposedWidget->InputState;
}

TSharedRef<SDockTab> FSlateIMExposedRoot::GetRootTab() const
{
	return RootTab;
}

TSharedRef<SWidget> FSlateIMExposedRoot::GetExposedWidget() const
{
	return ExposedWidget;
}
