// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMRoot.h"

class SImWrapper;

class FSlateIMExposedRoot : public ISlateIMRoot
{
	SLATE_IM_TYPE_DATA(FSlateIMExposedRoot, ISlateIMRoot)

public:
	FSlateIMExposedRoot();
	virtual ~FSlateIMExposedRoot() override;

	virtual void UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData) override;
	virtual bool IsVisible() const override { return true; }
	virtual FSlateIMInputState& GetInputState() override;
	virtual TSharedRef<SDockTab> GetRootTab() const override;
	
	TSharedRef<SWidget> GetExposedWidget() const;
	
private:
	TSharedRef<SImWrapper> ExposedWidget;
	TSharedRef<SDockTab> RootTab;
};
