// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_ENGINE
#include "SlateIMViewportRoot.h"

#if WITH_EDITOR
#include "IAssetViewport.h"
#endif

#include "SlateIM.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SImWrapper.h"

FSlateIMViewportRoot::FSlateIMViewportRoot(UGameViewportClient* InGameViewport)
	: GameViewport(InGameViewport)
	, RootTab(SNew(SDockTab))
{
	ensure(InGameViewport);
}

FSlateIMViewportRoot::FSlateIMViewportRoot(ULocalPlayer* InLocalPlayer)
	: GameViewport(InLocalPlayer ? InLocalPlayer->ViewportClient : nullptr)
	, LocalPlayer(InLocalPlayer)
	, RootTab(SNew(SDockTab))
{
	ensure(InLocalPlayer);
	ensure(GameViewport.IsValid());
}

#if WITH_EDITOR
FSlateIMViewportRoot::FSlateIMViewportRoot(TSharedPtr<IAssetViewport> InAssetViewport)
	: AssetViewport(InAssetViewport)
	, RootTab(SNew(SDockTab))
{
	ensure(InAssetViewport);
}
#endif // WITH_EDITOR

FSlateIMViewportRoot::~FSlateIMViewportRoot()
{
	if (ViewportRoot)
	{
#if WITH_EDITOR
		if (TSharedPtr<IAssetViewport> AssetViewportPinned = AssetViewport.Pin())
		{
			AssetViewportPinned->RemoveOverlayWidget(ViewportRoot.ToSharedRef());
		}
		else
#endif // WITH_EDITOR
		if (GameViewport.IsValid())
		{
			if (LocalPlayer.IsValid())
			{
				GameViewport->RemoveViewportWidgetForPlayer(LocalPlayer.Get(), ViewportRoot.ToSharedRef());
			}
			else
			{
				GameViewport->RemoveGameLayerWidget(ViewportRoot.ToSharedRef());
			}
		}
	}
}

void FSlateIMViewportRoot::UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData)
{
	if (Wrapper)
	{
		Wrapper->SetContent(Child);
	}
}

FSlateIMInputState& FSlateIMViewportRoot::GetInputState()
{
	check(Wrapper.IsValid());
	return Wrapper->InputState;
}

TSharedRef<SDockTab> FSlateIMViewportRoot::GetRootTab() const
{
	return RootTab;
}

void FSlateIMViewportRoot::UpdateViewport(const SlateIM::FViewportRootLayout& Layout)
{
	if (!ViewportRoot)
	{
		ViewportRoot = 
			SNew(SConstraintCanvas)
			.Visibility(EVisibility::SelfHitTestInvisible)
			+ SConstraintCanvas::Slot()
			.Anchors(Layout.Anchors)
			.Alignment(Layout.Alignment)
			.Expose(Slot)
			[
				SAssignNew(ScaleBox, SScaleBox)
				.Stretch(EStretch::UserSpecified)
				.UserSpecifiedScale(Layout.Scale)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(Wrapper, SImWrapper)	
				]
			];
		
#if WITH_EDITOR
		if (TSharedPtr<IAssetViewport> AssetViewportPinned = AssetViewport.Pin())
		{
			AssetViewportPinned->AddOverlayWidget(ViewportRoot.ToSharedRef(), Layout.ZOrder);
		}
		else
#endif // WITH_EDITOR
		if (GameViewport.IsValid())
		{
			if (LocalPlayer.IsValid())
			{
				GameViewport->AddViewportWidgetForPlayer(LocalPlayer.Get(), ViewportRoot.ToSharedRef(), Layout.ZOrder);
			}
			else
			{
				GameViewport->AddGameLayerWidget(ViewportRoot.ToSharedRef(), Layout.ZOrder);
			}
		}
	}
	else
	{
		Slot->SetAnchors(Layout.Anchors);
		Slot->SetAlignment(Layout.Alignment);
		ScaleBox->SetUserSpecifiedScale(Layout.Scale);
	}
	
	if (Layout.Size.IsSet())
	{
		Slot->SetAutoSize(false);
		Slot->SetOffset(FMargin(Layout.Offset.X, Layout.Offset.Y, Layout.Size->X, Layout.Size->Y));
	}
	else
	{
		Slot->SetAutoSize(true);
		Slot->SetOffset(Layout.Offset);
	}
}
#endif // WITH_ENGINE
