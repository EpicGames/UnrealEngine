// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ENGINE
#include "ISlateIMRoot.h"
#include "SlateIMParametersFwd.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Layout/SConstraintCanvas.h"

class SImWrapper;
class SScaleBox;

class UGameViewportClient;
class ULocalPlayer;

#if WITH_EDITOR
class IAssetViewport;
#endif

class FSlateIMViewportRoot : public ISlateIMRoot
{
	SLATE_IM_TYPE_DATA(FSlateIMViewportRoot, ISlateIMRoot)

public:
	FSlateIMViewportRoot(UGameViewportClient* InGameViewport);
	FSlateIMViewportRoot(ULocalPlayer* InLocalPlayer);
#if WITH_EDITOR
	FSlateIMViewportRoot(TSharedPtr<IAssetViewport> InLevelViewport);
#endif
	virtual ~FSlateIMViewportRoot() override;

	virtual void UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData) override;
	virtual bool IsVisible() const override
	{
		return true;
	}
	virtual FSlateIMInputState& GetInputState() override;
	virtual TSharedRef<SDockTab> GetRootTab() const override;

	void UpdateViewport(const SlateIM::FViewportRootLayout& Layout);

	TWeakObjectPtr<UGameViewportClient> GameViewport;
	TWeakObjectPtr<ULocalPlayer> LocalPlayer;
#if WITH_EDITOR
	TWeakPtr<IAssetViewport> AssetViewport;
#endif

private:
	TSharedPtr<SScaleBox> ScaleBox;
	TSharedPtr<SImWrapper> Wrapper;
	TSharedPtr<SWidget> ViewportRoot;
	SConstraintCanvas::FSlot* Slot = nullptr;
	TSharedRef<SDockTab> RootTab;
};
#endif // WITH_ENGINE
