// Copyright Epic Games, Inc. All Rights Reserved.


#include "SlateIMWindowRoot.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SImWrapper.h"

FSlateIMWindowRoot::FSlateIMWindowRoot(TSharedRef<SWindow> Window)
	: RootWindow(Window)
	, RootTab(SNew(SDockTab))
{}

FSlateIMWindowRoot::~FSlateIMWindowRoot()
{
	if (RootWindow.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RequestDestroyWindow(RootWindow.Pin().ToSharedRef());
		RootWindow = nullptr;
	}
	
	WindowRootWidget.Reset();
}

void FSlateIMWindowRoot::UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData)
{
	check(RootWindow != Child);

	if (!WindowRootWidget.IsValid())
	{
		WindowRootWidget = SNew(SImWrapper);
		RootWindow.Pin()->SetContent(WindowRootWidget.ToSharedRef());
	}

	WindowRootWidget->SetContent(Child);
}

bool FSlateIMWindowRoot::IsVisible() const
{
	return RootWindow.IsValid();
}

FSlateIMInputState& FSlateIMWindowRoot::GetInputState()
{
	check(WindowRootWidget.IsValid());
	return WindowRootWidget->InputState;
}

void FSlateIMWindowRoot::UpdateWindow(const FStringView& Title)
{
	if (TSharedPtr<SWindow> RootWindowPin = RootWindow.Pin())
	{
		FText NewWindowTitle = FText::FromStringView(Title);
		if (!RootWindowPin->GetTitle().IdenticalTo(NewWindowTitle))
		{
			RootWindowPin->SetTitle(NewWindowTitle);
		}
	}
}

TSharedPtr<SWindow> FSlateIMWindowRoot::GetWindow() const
{
	return RootWindow.Pin();
}

TSharedRef<SDockTab> FSlateIMWindowRoot::GetRootTab() const
{
	return RootTab;
}
