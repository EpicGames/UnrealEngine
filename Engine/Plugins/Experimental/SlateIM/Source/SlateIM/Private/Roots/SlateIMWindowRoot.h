// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateIMRoot.h"

class SDockTab;
class SImWrapper;
class SWindow;

class FSlateIMWindowRoot : public ISlateIMRoot
{
	SLATE_IM_TYPE_DATA(FSlateIMWindowRoot, ISlateIMRoot)

public:
	FSlateIMWindowRoot(TSharedRef<SWindow> Window);
	virtual ~FSlateIMWindowRoot() override;

	virtual void UpdateChild(TSharedRef<SWidget> Child, const FSlateIMSlotData& AlignmentData) override;
	virtual bool IsVisible() const override;
	virtual FSlateIMInputState& GetInputState() override;
	virtual TSharedRef<SDockTab> GetRootTab() const override;

	void UpdateWindow(const FStringView& Title);

	TSharedPtr<SWindow> GetWindow() const;
	
private:
	TWeakPtr<SWindow> RootWindow;
	TSharedPtr<SImWrapper> WindowRootWidget;
	TSharedRef<SDockTab> RootTab;
};
