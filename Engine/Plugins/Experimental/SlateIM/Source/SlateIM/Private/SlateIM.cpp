// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIM.h"

#include "Misc/SlateIMLogging.h"
#include "Misc/SlateIMManager.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogSlateIM);

void FSlateIMModule::StartupModule()
{
	SlateIM::FSlateIMManager::Initialize();
}

void FSlateIMModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FSlateIMModule, SlateIM)
