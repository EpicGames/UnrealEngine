// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

FSlateIMStyle::FSlateIMStyle()
	: FSlateStyleSet(TEXT("SlateIM"))
{
	const FName ButtonStyleName = "WindowMenuBar.Button";

	FButtonStyle MenuButtonStyle = FAppStyle::GetWidgetStyle<FButtonStyle>(ButtonStyleName);
	MenuButtonStyle.NormalPadding.Left = 6.f;
	MenuButtonStyle.NormalPadding.Right = 6.f;
	MenuButtonStyle.PressedPadding.Left = 6.f;
	MenuButtonStyle.PressedPadding.Right = 6.f;

	Set("SlateIM.MenuButton", MenuButtonStyle);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSlateIMStyle::~FSlateIMStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
