// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FSlateIMStyle final : public FSlateStyleSet
{
public:
	static FSlateIMStyle& Get()
	{
		static FSlateIMStyle Instance;
		return Instance;
	}

	FSlateIMStyle();
	virtual ~FSlateIMStyle() override;
};
