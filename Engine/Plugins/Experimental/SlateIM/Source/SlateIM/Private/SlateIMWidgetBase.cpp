// Copyright Epic Games, Inc. All Rights Reserved.


#include "SlateIMWidgetBase.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Framework/Docking/TabManager.h"
#include "SlateIM.h"
#include "Misc/SlateIMLogging.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNullWidget.h"

FSlateIMWidgetBase::FSlateIMWidgetBase(const FStringView& Name)
	: WidgetName(Name)
{
}

FSlateIMWidgetBase::~FSlateIMWidgetBase()
{
	if (FSlateApplication::IsInitialized())
	{
		DisableWidget();
	}
}

void FSlateIMWidgetBase::EnableWidget()
{
	if (!TickHandle.IsValid())
	{
		TickHandle = FSlateApplication::Get().OnPreTick().AddRaw(this, &FSlateIMWidgetBase::DrawWidget);
	}
}

void FSlateIMWidgetBase::DisableWidget()
{
	if (TickHandle.IsValid())
	{
		FSlateApplication::Get().OnPreTick().Remove(TickHandle);
		TickHandle.Reset();
	}
}

FSlateIMWidgetWithCommandBase::FSlateIMWidgetWithCommandBase(const TCHAR* Command, const TCHAR* CommandHelp)
	: FSlateIMWidgetBase(Command)
	, WidgetCommand(Command, CommandHelp, FConsoleCommandDelegate::CreateRaw(this, &FSlateIMWidgetWithCommandBase::ToggleWidget))
{}

FSlateIMWidgetWithCommandBase::FSlateIMWidgetWithCommandBase(const FStringView& WidgetName, const TCHAR* Command, const TCHAR* CommandHelp)
	: FSlateIMWidgetBase(WidgetName)
	, WidgetCommand(Command, CommandHelp, FConsoleCommandDelegate::CreateRaw(this, &FSlateIMWidgetWithCommandBase::ToggleWidget))
{}

void FSlateIMWindowBase::DrawWidget(float DeltaTime)
{
	if (!SlateIM::CanUpdateSlateIM())
	{
		return;
	}
	
	const bool bIsDrawingWindow = SlateIM::BeginWindowRoot(
		GetWidgetName(),
		{
			.WindowTitle  = WindowTitle,
			.WindowSize   = WindowSize,
			.bAlwaysOnTop = bAlwaysOnTop
		}
	);

	if (bIsDrawingWindow)
	{
		DrawWindow(DeltaTime);
	}
	SlateIM::EndRoot();

	if (!bIsDrawingWindow)
	{
		DisableWidget();
	}
}

FSlateIMNomadTabBase::FSlateIMNomadTabBase(FStringView TabTitle, const TCHAR* Command, const TCHAR* CommandHelp, FSlateIcon TabIcon)
	: FSlateIMWidgetWithCommandBase(TabTitle, Command, CommandHelp)
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(GetWidgetName(), FOnSpawnTab::CreateLambda([this, TabTitleText = FText::FromStringView(TabTitle)](const FSpawnTabArgs&)
	{
		EnableWidget();

		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.TabRole(ETabRole::NomadTab)
			.Label(TabTitleText)
			.OnTabClosed_Lambda([this](TSharedRef<SDockTab>)
				{ FSlateIMWidgetWithCommandBase::DisableWidget(); TabContent.Reset(); });
	}))
		.SetIcon(TabIcon);
}

FSlateIMNomadTabBase::~FSlateIMNomadTabBase()
{
	if (TSharedPtr<SDockTab> RootTabPin = RootTab.Pin())
	{
		RootTabPin->SetOnTabActivated(SDockTab::FOnTabActivatedCallback());
		RootTabPin->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
		RootTabPin->RequestCloseTab();
		RootTab.Reset();
		TabContent.Reset();
	}
	
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(GetWidgetName());
}

void FSlateIMNomadTabBase::DisableWidget()
{
	if (TSharedPtr<SDockTab> RootTabPin = RootTab.Pin())
	{
		RootTabPin->RequestCloseTab();
		RootTab.Reset();
	}
	
	FSlateIMWidgetWithCommandBase::DisableWidget();
}

void FSlateIMNomadTabBase::DrawWidget(float DeltaTime)
{
	if (!SlateIM::CanUpdateSlateIM())
	{
		return;
	}

	TSharedPtr<SDockTab> RootTabPin = RootTab.Pin();
	if (!RootTabPin.IsValid())
	{
		// This is the first time we call DrawWidget() since being enabled - invoke the tab
		RootTab = RootTabPin = FGlobalTabmanager::Get()->TryInvokeTab(GetWidgetName());
		if (!RootTabPin.IsValid())
		{
			UE_LOGF(LogSlateIM, Error, "Cannot invoke tab '%ls'; disabling widget", *GetWidgetName().ToString());
			DisableWidget();
			return;
		}
	}

	TSharedPtr<SWidget> NewExposedWidget;
	if (SlateIM::BeginExposedRoot(GetWidgetName(), NewExposedWidget))
	{
		DrawContent(DeltaTime);
	}
	SlateIM::EndRoot();

	if (NewExposedWidget.IsValid())
	{
		if (NewExposedWidget != TabContent)
		{
			RootTabPin->SetContent(NewExposedWidget.ToSharedRef());
			TabContent = NewExposedWidget;
		}
	}
	else
	{
		RootTabPin->RequestCloseTab();
		TabContent.Reset();
		DisableWidget();
	}
}

TSharedRef<SWidget> FSlateIMExposedBase::GetExposedWidget() const
{
	return ExposedWidget.IsValid() ? ExposedWidget.ToSharedRef() : SNullWidget::NullWidget;
}

void FSlateIMExposedBase::DrawWidget(float DeltaTime)
{
	TSharedPtr<SWidget> NewExposedWidget;
	if (SlateIM::BeginExposedRoot(GetWidgetName(), NewExposedWidget))
	{
		DrawContent(DeltaTime);
	}
	SlateIM::EndRoot();

	if (NewExposedWidget != ExposedWidget)
	{
		ExposedWidget = NewExposedWidget;
		OnExposedWidgetChanged.Broadcast(GetExposedWidget());
	}
}
