// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "SlateIMParametersFwd.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointerFwd.h"

class SImage;
enum EOrientation : int;
struct FSlateBrush;
struct FSlateColor;

namespace SlateIM
{
	void BeginContainer();
	void EndContainer();

	void BeginStack(EOrientation Orientation);
	void EndStack();

	void BeginWrap(EOrientation Orientation);
	void EndWrap();

	void Image_Internal(const FImageParams& Params, TFunctionRef<const FSlateBrush* (const TSharedRef<SImage>&)> GetBrush);
} // SlateIM
