// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImButton.h"

#include "Widgets/Text/STextBlock.h"

SLATE_IMPLEMENT_WIDGET(SImButton)

void SImButton::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SImButton::Construct(const FArguments& InArgs)
{
	SButton::FArguments ButtonArgs;

	TextBox = SNew(STextBlock)
		.Visibility(EVisibility::HitTestInvisible)
		.TextStyle(ButtonArgs._TextStyle)
		.TextShapingMethod(ButtonArgs._TextShapingMethod)
		.TextFlowDirection(ButtonArgs._TextFlowDirection);

	ButtonArgs.OnClicked(InArgs._OnClicked);

	ButtonArgs
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		TextBox.ToSharedRef()
	];

	SButton::Construct(ButtonArgs);
}

void SImButton::SetText(const FStringView& InText)
{
	TextBox->SetText(FText::FromStringView(InText));
}
