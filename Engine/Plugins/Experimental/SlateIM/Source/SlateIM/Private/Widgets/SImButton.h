// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SButton.h"


class STextBlock;

class SImButton : public SButton
{
	SLATE_DECLARE_WIDGET(SImButton, SButton)

public:
	SLATE_BEGIN_ARGS(SImButton) {}
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetText(const FStringView& InText);

private:
	TSharedPtr<STextBlock> TextBox;
};
