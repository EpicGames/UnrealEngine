// Copyright Epic Games, Inc. All Rights Reserved.

#include "SImCheckBox.h"

#include "Widgets/Text/STextBlock.h"

SLATE_IMPLEMENT_WIDGET(SImCheckBox)

void SImCheckBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SImCheckBox::Construct(const FArguments& InArgs)
{
	TextBox = SNew(STextBlock).Visibility(EVisibility::HitTestInvisible);
	SCheckBox::Construct(
		SCheckBox::FArguments()
		.IsChecked(InArgs._IsChecked)
		.OnCheckStateChanged(InArgs._OnCheckStateChanged)
		[
			TextBox.ToSharedRef()
		]
	);
}

void SImCheckBox::SetText(const FText& InText)
{
	TextBox->SetText(InText);
}
