// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SCheckBox.h"

class STextBlock;

class SImCheckBox : public SCheckBox
{
	SLATE_DECLARE_WIDGET(SImCheckBox, SCheckBox)
	
public:
	SLATE_BEGIN_ARGS(SImCheckBox) {}
		SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)
		SLATE_EVENT(FOnCheckStateChanged, OnCheckStateChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetText(const FText& InText);

private:
	TSharedPtr<STextBlock> TextBox;
};