// Copyright Epic Games, Inc. All Rights Reserved.


#include "SImGraph.h"
#include "SlateIMParameters.h"

SLATE_IMPLEMENT_WIDGET(SImGraph)

void SImGraph::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SImGraph::Construct(const FArguments& InArgs)
{
	SetClipping(EWidgetClipping::ClipToBounds);
}

void SImGraph::BeginGraph()
{
	for (FSlateIMLineGraphData& LineGraph : LineGraphs)
	{
		LineGraph.bIsStale = true;
	}
}

void SImGraph::EndGraph()
{
	const int32 FirstStaleIndex = LineGraphs.IndexOfByPredicate([](const FSlateIMLineGraphData& LineGraph)
	{
		return LineGraph.bIsStale;
	});

	if (FirstStaleIndex != INDEX_NONE)
	{
		LineGraphs.SetNum(FirstStaleIndex, EAllowShrinking::No);
	}
}

void SImGraph::AddLineGraph(const TArrayView<FVector2D>& Points, const SlateIM::FGraphLinePointsParams& Params)
{
	ensure(Params.XViewRange.HasLowerBound() && Params.XViewRange.HasUpperBound());
	ensure(Params.YViewRange.HasLowerBound() && Params.YViewRange.HasUpperBound());
	
	FSlateIMLineGraphData& LineGraph = GetNextLineGraph(Params.LineColor, Params.LineThickness, Params.XViewRange, Params.YViewRange);
	LineGraph.NormalizedPoints.Reserve(Points.Num());

	for (const FVector2D& Point : Points)
	{
		LineGraph.NormalizedPoints.Emplace(
			(Point.X - Params.XViewRange.GetLowerBound().GetValue()) / (Params.XViewRange.GetUpperBound().GetValue() - Params.XViewRange.GetLowerBound().GetValue()),
			(Point.Y - Params.YViewRange.GetLowerBound().GetValue()) / (Params.YViewRange.GetUpperBound().GetValue() - Params.YViewRange.GetLowerBound().GetValue()));
	}
}

void SImGraph::AddLineGraph(const TArrayView<double>& Values, const SlateIM::FGraphLineValuesParams& Params)
{
	ensure(Params.ViewRange.HasLowerBound() && Params.ViewRange.HasUpperBound());

	const FDoubleRange XViewRange = FDoubleRange(0.0, static_cast<double>(Values.Num()));
	
	FSlateIMLineGraphData& LineGraph = GetNextLineGraph(Params.LineColor, Params.LineThickness, XViewRange, Params.ViewRange);
	LineGraph.NormalizedPoints.Reserve(Values.Num());

	double XValue = 0.0;
	for (double Value : Values)
	{
		LineGraph.NormalizedPoints.Emplace(
			(XValue - XViewRange.GetLowerBound().GetValue()) / (XViewRange.GetUpperBound().GetValue() - XViewRange.GetLowerBound().GetValue()),
			(Value - Params.ViewRange.GetLowerBound().GetValue()) / (Params.ViewRange.GetUpperBound().GetValue() - Params.ViewRange.GetLowerBound().GetValue()));

		XValue += 1.0;
	}
}

int32 SImGraph::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector2D WidgetSize = AllottedGeometry.GetLocalSize();

	TArray<FVector2D> Points;
	for (const FSlateIMLineGraphData& LineGraph : LineGraphs)
	{
		Points.Reset(LineGraph.NormalizedPoints.Num());

		for (const FVector2D& NormalizedPoint : LineGraph.NormalizedPoints)
		{
			Points.Emplace(WidgetSize.X * NormalizedPoint.X, WidgetSize.Y - WidgetSize.Y * NormalizedPoint.Y);
		}
		
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			Points,
			ESlateDrawEffect::NoPixelSnapping,
			LineGraph.Color,
			true,
			LineGraph.LineThickness);
	}
	
	return LayerId;
}

FVector2D SImGraph::ComputeDesiredSize(float) const
{
	// Graph size should be set by the SlateIM slot, so just return a reasonable default
	return FVector2D(300, 200);
}

FSlateIMLineGraphData& SImGraph::GetNextLineGraph(const FLinearColor& LineColor, float LineThickness, const FDoubleRange& XViewRange, const FDoubleRange& YViewRange)
{
	for (FSlateIMLineGraphData& ExistingLineGraph : LineGraphs)
	{
		if (ExistingLineGraph.bIsStale)
		{
			ExistingLineGraph.NormalizedPoints.Reset();
			ExistingLineGraph.Color = LineColor;
			ExistingLineGraph.LineThickness = LineThickness;
			ExistingLineGraph.XViewRange = XViewRange;
			ExistingLineGraph.YViewRange = YViewRange;
			ExistingLineGraph.bIsStale = false;
				
			return ExistingLineGraph;
		}
	}

	FSlateIMLineGraphData NewLineGraph;
	NewLineGraph.Color = LineColor;
	NewLineGraph.LineThickness = LineThickness;
	NewLineGraph.XViewRange = XViewRange;
	NewLineGraph.YViewRange = YViewRange;

	return LineGraphs.Emplace_GetRef(MoveTemp(NewLineGraph));
}
