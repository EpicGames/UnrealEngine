// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateIMParametersFwd.h"
#include "Widgets/SLeafWidget.h"

#define UE_API SLATEIM_API

struct FSlateIMLineGraphData
{
	TArray<FVector2D> NormalizedPoints;
	FLinearColor Color = FLinearColor::White;
	float LineThickness = 1.0f;
	FDoubleRange XViewRange;
	FDoubleRange YViewRange;
	bool bIsStale = false;
};

class SImGraph : public SLeafWidget
{
	SLATE_DECLARE_WIDGET(SImGraph, SLeafWidget)
	
public:
	SLATE_BEGIN_ARGS(SImGraph) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API void BeginGraph();
	UE_API void EndGraph();

	UE_API void AddLineGraph(const TArrayView<FVector2D>& Points, const SlateIM::FGraphLinePointsParams& Params);
	UE_API void AddLineGraph(const TArrayView<double>& Values, const SlateIM::FGraphLineValuesParams& Params);

private:
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	FSlateIMLineGraphData& GetNextLineGraph(const FLinearColor& LineColor, float LineThickness, const FDoubleRange& XViewRange, const FDoubleRange& YViewRange);
	
	TArray<FSlateIMLineGraphData> LineGraphs;
};

#undef UE_API
