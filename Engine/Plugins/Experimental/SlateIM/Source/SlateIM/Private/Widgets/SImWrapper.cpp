// Copyright Epic Games, Inc. All Rights Reserved.


#include "SImWrapper.h"


SImWrapper::~SImWrapper()
{
}

void SImWrapper::Construct(const FArguments& InArgs)
{
	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		InArgs._Content.Widget
	];
}

void SImWrapper::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	// Inputs are only considered to be released for 1 frame
	for (auto It = InputState.KeyStateMap.CreateIterator(); It; ++It)
	{
		if (It.Value() == ESlateIMKeyState::Released)
		{
			It.Value() = ESlateIMKeyState::Idle;
		}
	}

	// Reset mouse wheel axis since we don't get "0" events for it
	if (float* MouseWheelValuePtr = InputState.AnalogValueMap.Find(EKeys::MouseWheelAxis))
	{
		*MouseWheelValuePtr = 0;
	}
}

bool SImWrapper::SupportsKeyboardFocus() const
{
	return true;
}

FReply SImWrapper::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	InputState.KeyStateMap.FindOrAdd(KeyEvent.GetKey()) = (KeyEvent.IsRepeat() ? ESlateIMKeyState::Held : ESlateIMKeyState::Pressed);
	return SCompoundWidget::OnKeyDown(MyGeometry, KeyEvent);
}

FReply SImWrapper::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	InputState.KeyStateMap.FindOrAdd(KeyEvent.GetKey()) = ESlateIMKeyState::Released;
	return SCompoundWidget::OnKeyUp(MyGeometry, KeyEvent);
}

FReply SImWrapper::OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& AnalogInputEvent)
{
	InputState.AnalogValueMap.FindOrAdd(AnalogInputEvent.GetKey()) = AnalogInputEvent.GetAnalogValue();
	return SCompoundWidget::OnAnalogValueChanged(MyGeometry, AnalogInputEvent);
}

FReply SImWrapper::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	InputState.KeyStateMap.FindOrAdd(MouseEvent.GetEffectingButton()) = (MouseEvent.IsRepeat() ? ESlateIMKeyState::Held : ESlateIMKeyState::Pressed);
	return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SImWrapper::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	InputState.KeyStateMap.FindOrAdd(MouseEvent.GetEffectingButton()) = (MouseEvent.IsRepeat() ? ESlateIMKeyState::Held : ESlateIMKeyState::Pressed);
	return SCompoundWidget::OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
}

FReply SImWrapper::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	InputState.KeyStateMap.FindOrAdd(MouseEvent.GetEffectingButton()) = ESlateIMKeyState::Released;
	return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SImWrapper::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	InputState.AnalogValueMap.FindOrAdd(EKeys::MouseWheelAxis) = MouseEvent.GetWheelDelta();
	return SCompoundWidget::OnMouseWheel(MyGeometry, MouseEvent);
}

FReply SImWrapper::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	InputState.AnalogValueMap.FindOrAdd(EKeys::MouseX) = MouseEvent.GetCursorDelta().X;
	InputState.AnalogValueMap.FindOrAdd(EKeys::MouseY) = MouseEvent.GetCursorDelta().Y;
	return SCompoundWidget::OnMouseMove(MyGeometry, MouseEvent);
}

void SImWrapper::SetContent(const TSharedRef<SWidget>& InContent)
{
	ChildSlot
	[
		InContent
	];
}

void SImWrapper::SetHAlign(EHorizontalAlignment HAlign)
{
	ChildSlot.SetHorizontalAlignment(HAlign);
}

void SImWrapper::SetVAlign(EVerticalAlignment VAlign)
{
	ChildSlot.SetVerticalAlignment(VAlign);
}

void SImWrapper::SetPadding(TAttribute<FMargin> InPadding)
{
	ChildSlot.SetPadding(InPadding);
}
