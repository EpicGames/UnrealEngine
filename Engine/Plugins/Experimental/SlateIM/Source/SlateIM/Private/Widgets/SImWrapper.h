// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/SlateIMInputState.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SLATEIM_API

class SImWrapper : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SImWrapper)
		{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UE_API virtual ~SImWrapper() override;

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API virtual bool SupportsKeyboardFocus() const override;

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;
	UE_API virtual FReply OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& AnalogInputEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API void SetContent(const TSharedRef< SWidget >& InContent);
	UE_API void SetHAlign(EHorizontalAlignment HAlign);
	UE_API void SetVAlign(EVerticalAlignment VAlign);
	UE_API void SetPadding(TAttribute<FMargin> InPadding);

	FSlateIMInputState InputState;
};

#undef UE_API
