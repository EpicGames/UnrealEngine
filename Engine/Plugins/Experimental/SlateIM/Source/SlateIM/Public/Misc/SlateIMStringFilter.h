// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Misc/TextFilter.h"

/**
 * Base template class for SlateIM text filters.
 *
 * Provides a common interface for filtering items using TTextFilter.
 */
template<typename ItemType>
class FSlateImTextFilter
{
public:
	FSlateImTextFilter() = default;

	virtual ~FSlateImTextFilter() = default;

	/**
	 * Filters an array of items, returning a new array with only matching entries.
	 * @param InArray The source array to filter
	 * @return A new array containing only the items that pass the filter
	 */
	using ArrayItemType = std::remove_const_t<typename TRemoveReference<ItemType>::Type>;
	TArray<ArrayItemType> Filter(const TArray<ArrayItemType>& InArray) const
	{
		if (!TextFilter.IsValid())
		{
			return InArray;
		}

		return InArray.FilterByPredicate([this](const ArrayItemType& Item)
		{
			return TextFilter->PassesFilter(Item);
		});
	}

	FString InputText;

	TSharedPtr<TTextFilter<ItemType>> TextFilter;
};

/**
 * SlateIM filter utility class for filtering FStrings in immediate-mode menus.
 * Usage:
 *   // As a member variable in your SlateIM menu class
 *   FSlateIMStringFilter MyFilter;
 *
 *   // In your immediate-mode drawing code
 *   SlateIM::TextFilter(MyFilter, TEXT("Filter"));
 *
 *   // Filter an array of items
 *   TArray<FString> FilteredItems = MyFilter.Filter(Items);
 */
class FSlateIMStringFilter : public FSlateImTextFilter<const FString&>
{
public:
	FSlateIMStringFilter()
	{
		// Create a TTextFilter for filtering FStrings
		// The transform delegate simply returns the string itself in an array
		TextFilter = MakeShared<TTextFilter<const FString&>>(
			TTextFilter<const FString&>::FItemToStringArray::CreateLambda([](const FString& InString, TArray<FString>& OutStrings)
			{
				OutStrings.Add(InString);
			})
		);
	}
};
