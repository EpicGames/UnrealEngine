// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Views/ITypedTableView.h"
#include "Layout/Margin.h"
#include "Modules/ModuleInterface.h"
#include "SlateIMParameters.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/Anchors.h"
#if WITH_ENGINE
#include "Engine/EngineTypes.h"
#endif

#if WITH_EDITOR
class IAssetViewport;
#endif
#if WITH_ENGINE
class FCanvasItem;
class UCanvas;
class UFont;
class UGameViewportClient;
class ULocalPlayer;
class UMaterialInterface;
class UTextureRenderTarget2D;
class UTexture;
class UTexture2D;
struct FCanvasIcon;
#endif

struct FButtonStyle;
struct FCheckBoxStyle;
struct FComboBoxStyle;
struct FEditableTextBoxStyle;
struct FKey;
struct FProgressBarStyle;
struct FSlateBrush;
struct FSliderStyle;
struct FSpinBoxStyle;
struct FTableRowStyle;
struct FTableViewStyle;
struct FTextBlockStyle;

enum class ECheckBoxState : uint8;

class SWidget;
template<typename ItemType> class FSlateImTextFilter;


class FSlateIMModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

/**
 * This is the full list of available functions for creating SlateIM widgets. See FSlateIMTestWidget::Draw() for examples.
 */
namespace SlateIM
{
#pragma region Roots
	/**
	 * Begins a new floating window root
	 *
	 * Please use the method below with the struct parameter.
	 *
	 * @param UniqueName a unique name to identify this window
	 * @param WindowTitle The name of the window. This will also be displayed in the windows title bar
	 * @param WindowSize The size to open the window at (does not update the size of an existing window)
	 * @param bShouldReopen (optional) If this window was created with a previous call to BeginWindow and then closed, passing in true will reopen the window and false will leave it closed. If the window has never been seen this parameter does nothing and the window will open
	 *
	 * @return the current window state. True if open, false if closed or not updating
	 *
	 * @note Consider using FSlateIMWindowBase to handle creating and updating your Slate IM Window
	 */
	SLATEIM_API bool BeginWindowRoot(FName UniqueName, const FStringView& WindowTitle, FVector2f WindowSize, bool bShouldReopen = false);

	/**
	 * Begins a new floating window root
	 *
	 * @param UniqueName a unique name to identify this window
	 * @param Params (optional) Additional parameters to create the window.
	 *
	 * @return the current window state. True if open, false if closed or not updating
	 *
	 * @note Consider using FSlateIMWindowBase to handle creating and updating your Slate IM Window
	 */
	SLATEIM_API bool BeginWindowRoot(FName UniqueName, const FWindowParams& Params = {});

#if WITH_ENGINE
	// TODO - Game viewport widgets might need something to control HitTestability and cursor visibility

	/**
	 * Begins a new root in the game viewport
	 *
	 * Please use the method below with the struct-parameter.
	 *
	 * @param UniqueName a globally unique name to identify this root
	 * @param ViewportClient the viewport client to add the widget to
	 * @param Layout How to lay out the root within the viewport
	 *
	 * @return Whether the root is valid and updating
	 */
	SLATEIM_API bool BeginViewportRoot(FName UniqueName, UGameViewportClient* ViewportClient, const FViewportRootLayout& Layout);

	/**
	 * Begins a new root in the game viewport
	 *
	 * @param UniqueName a globally unique name to identify this root
	 * @param ViewportClient the viewport client to add the widget to
	 * @param Params (optional) Additional parameters to create the root.
	 *
	 * @return Whether the root is valid and updating
	 */
	SLATEIM_API bool BeginViewportRoot(FName UniqueName, UGameViewportClient* ViewportClient, const FViewportParams& Params = FViewportParams());
	
	/**
	 * Begins a new root in the provided player's viewport
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param UniqueName a globally unique name to identify this root
	 * @param LocalPlayer the player whose viewport to add the widget to
	 * @param Layout How to lay out the root within the viewport
	 * 
	 * @return Whether the root is valid and updating
	 */
	SLATEIM_API bool BeginViewportRoot(FName UniqueName, ULocalPlayer* LocalPlayer, const FViewportRootLayout& Layout);
	
	/**
	 * Begins a new root in the provided player's viewport
	 * 
	 * @param UniqueName a globally unique name to identify this root
	 * @param LocalPlayer the player whose viewport to add the widget to
	 * @param Params (optional) Additional parameters to create the root.
	 * 
	 * @return Whether the root is valid and updating
	 */
	SLATEIM_API bool BeginViewportRoot(FName UniqueName, ULocalPlayer* LocalPlayer, const FViewportParams& Params = FViewportParams());
#endif

#if WITH_EDITOR
	/**
	 * Begins a SlateIM root in the provided in editor viewport
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param UniqueName Unique identifier for this root. Must be globally unique within the same editor session
	 * @param AssetViewport The editor viewpor to add the SlateIM content to
	 * @param Layout How to lay out the root within the viewport
	 * 
	 * @return Whether the root is valid and updating
	 *
	 * @note Not available in BPFL: TSharedPtr<IAssetViewport> is not Blueprint-compatible
	 */
	SLATEIM_API bool BeginViewportRoot(FName UniqueName, TSharedPtr<IAssetViewport> AssetViewport, const FViewportRootLayout& Layout);

	/**
	 * Begins a SlateIM root in the provided in editor viewport
	 *
	 * @param UniqueName Unique identifier for this root. Must be globally unique within the same editor session
	 * @param AssetViewport The editor viewpor to add the SlateIM content to
	 * @param Params (optional) Additional parameters to create the root.
	 *
	 * @return Whether the root is valid and updating
	 *
	 * @note Not available in BPFL: TSharedPtr<IAssetViewport> is not Blueprint-compatible
	 */
	SLATEIM_API bool BeginViewportRoot(FName UniqueName, TSharedPtr<IAssetViewport> AssetViewport, const FViewportParams& Params = FViewportParams());
#endif

	/**
	 * Begins a SlateIM root that exposes a slate widget for embedding in other Slate hierarchies
	 *
	 * @param UniqueName a globally unique name to identify this root
	 * @param OutSlateIMWidget where to output the resulting SlateIM widget to
	 *
	 * @return Whether the root is valid and updating
	 *
	 * @note Consider using FSlateIMExposedBase to handle creating and updating your Slate IM exposed widget
	 * @note Not available in BPFL: TSharedPtr<SWidget> is not Blueprint-compatible
	 */
	SLATEIM_API bool BeginExposedRoot(FName UniqueName, TSharedPtr<SWidget>& OutSlateIMWidget);

	/**
	 * Ends any Root type, must always be called regardless of the result of the Begin function
	 */
	SLATEIM_API void EndRoot();
#pragma endregion Roots

#pragma region Containers
	/**
	 * Begins a horizontally stacked container. All widgets created within the Horizontal Stack container will be placed in-order left-to-right
	 * 
	 * @param bMaximizeContent (optional) Maximize() the contents of the stack
	 */
	SLATEIM_API void BeginHorizontalStack(bool bMaximizeContent = false);

	SLATEIM_API void EndHorizontalStack();

	/**
	 * Begins a vertically stacked container. All widgets created within the Vertical Stack container will be placed in-order top-to-bottom
	 *
	 * @param bMaximizeContent (optional) Maximize() the contents of the stack
	 */
	SLATEIM_API void BeginVerticalStack(bool bMaximizeContent = false);
	SLATEIM_API void EndVerticalStack();

	/**
	 * Begins a horizontally wrapped container. All widgets created within the Horizontal Wrap container will be placed in-order left-to-right until the allotted width is filled,
	 * then the content will begin a new line and so-on.
	 *
	 * @param bMaximizeContent (optional) Maximize() the contents of the wrap
	 *
	 * @note Wrap box children are always AutoSize and do not support Fill
	 */
	SLATEIM_API void BeginHorizontalWrap(bool bMaximizeContent = false);

	SLATEIM_API void EndHorizontalWrap();
	
	/**
	 * Begins a vertically wrapped container. All widgets created within the Vertical Wrap container will be placed in-order top-to-bottom until the allotted height is filled,
	 * then the content will begin a new column and so-on.
	 *
	 * @param bMaximizeContent (optional) Maximize() the contents of the wrap
	 *
	 * @note Wrap container children are always AutoSize and do not support Fill
	 */
	SLATEIM_API void BeginVerticalWrap(bool bMaximizeContent = false);

	SLATEIM_API void EndVerticalWrap();

	/**
	 * Begin a Table container. The contents are automatically scrollable if the table is sized smaller than all its content.
	 *
	 * Please use the method below with the struct-parameter.
	 *
	 * @param InStyle A Table View Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *				  provide a new style object to update the visuals at runtime
	 * @param InRowStyle (optional) A Table Row Style to override the default table row style. Changing the style only affects new table rows.
	 */
	SLATEIM_API void BeginTable(const FTableViewStyle* InStyle, const FTableRowStyle* InRowStyle = nullptr);

	/**
	 * Begin a Table container. The contents are automatically scrollable if the table is sized smaller than all its content.
	 *
	 * @param Params (optional) Additional parameters to create the table.
	 */
	SLATEIM_API void BeginTable(const FTableParams& Params = FTableParams());

	/**
	 * End the current Table container.
	 */
	SLATEIM_API void EndTable();

	/**
	 * Starts the header. Call this before adding any content to the table.
	 */
	SLATEIM_API void BeginTableHeader();

	/**
	 * Ends the table header.
	 */
	SLATEIM_API void EndTableHeader();

	/**
	 * Moves to the next column. Allows fine control of the column body.
	 *
	 * @param ColumnID The name to use as ID for the column
	 *
	 * @note Table columns support SetToolTip, HAlign, and VAlign. See FixedTableColumnWidth() and InitialTableColumnWidth() for size options.
	 */
	SLATEIM_API void NextTableColumn(const FName& ColumnID);

	/**
	 * Adds a column and immediately ends it with the given label as its body.
	 *
	 * Please use the method below with the struct-parameter.
	 *
	 * @param ColumnID The name to use as ID for the column
	 * @param ColumnLabel Added a default label to the column
	 */
	SLATEIM_API void AddTableColumn(const FName& ColumnID, const FStringView& ColumnLabel);

	/**
	 * Adds a column and immediately ends it with the given label as its body.
	 *
	 * @param ColumnID The name to use as ID for the column
	 * @param Params (optional) Additional parameters to create the column
	 */
	SLATEIM_API void AddTableColumn(const FName& ColumnID, const FTableColumnParams& Params = FTableColumnParams());

	/**
	 * Set the column to a fixed width, not resizable by users. Call this before calling AddTableColumn().
	 * 
	 * @param Width The fixed width of the column
	 */
	SLATEIM_API void FixedTableColumnWidth(float Width);

	/**
	 * Set the column to an initial width, which can then be resized by users. Call this before calling AddTableColumn().
	 * 
	 * @param Width The initial width of the column
	 */
	SLATEIM_API void InitialTableColumnWidth(float Width);

	/**
	 * Starts a table body.
	 */
	SLATEIM_API void BeginTableBody();

	/**
	 * Ends a table body.
	 */
	SLATEIM_API void EndTableBody();

	/**
	 * Begin the content for the next table cell. The column and row are tracked internally based on the number of columns.
	 * 
	 * @param bOutRowSelected (optional) If supplied, will return true if the row this cell belongs to is selected.
	 * 
	 * @return Whether the contents of this table cell are visible. When false, the content can be skipped to save cycles.
	 *
	 * @note There is no "End" function counterpart to this call. This function supports the various slot functions (Padding, HAlign, VAlign, Min/Max Width/Height, etc).
	 */
	SLATEIM_API bool NextTableCell(bool* bOutRowSelected = nullptr);

	/**
	 * Begin adding content as a child to the current table row. NextTableCell() must still be called after this function.
	 * If the table is a tree, a unique row id must be provided when starting a new set of child rows so that the expansion
	 * state of the parent row is saved. Not providing a row id will cause the parent row to revert to the default expansion
	 * state if its grandparent row is closed.
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ParentRowId Id of the parent row. 0 to disable state saving. Row ids must be unique to its parent.
	 * @param bDefaultExpanded (optional) Set the default expansion state of the parent (whether the children following are visible.)
	 *   Only applies to rows with a parent id.
	 * 
	 * @return Whether the parent row is expanded or not. When false, the child content can be skipped to save cycles.
	 */
	SLATEIM_API bool BeginTableRowChildren(uint32 ParentRowId, bool bDefaultExpanded = false);

	/**
	 * Begin adding content as a child to the current table row. NextTableCell() must still be called after this function.
	 * If the table is a tree, a unique row id must be provided when starting a new set of child rows so that the expansion
	 * state of the parent row is saved. Not providing a row id will cause the parent row to revert to the default expansion
	 * state if its grandparent row is closed.
	 *
	 * @param Params (optional) Additional parameters to create the table children.
	 *
	 * @return Whether the parent row is expanded or not. When false, the child content can be skipped to save cycles.
	 */
	SLATEIM_API bool BeginTableRowChildren(const FTableRowChildrenParams& Params = FTableRowChildrenParams());
	
	/**
	 * Stop adding child content to the parent row. This must be called no matter the result of BeginTableRowChildren()
	 */
	SLATEIM_API void EndTableRowChildren();

	/**
	 * Begin a container with a background image.
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param BorderBrush The brush to use as the background for this container
	 * @param Orientation Which direction the contents of the container should flow
	 * @param bAbsorbMouse (optional) Whether the container should handle all mouse inputs
	 * @param ContentPadding (optional) How much to pad the contents of the container
	 */
	SLATEIM_API void BeginBorder(const FSlateBrush* BorderBrush, EOrientation Orientation, bool bAbsorbMouse = true, FMargin ContentPadding = FMargin(2.0f));
	
	/**
	 * Begin a container with a background image.
	 *
	 * @param BorderBrush The brush to use as the background for this container
	 * @param Params (optional) Additional params to create the border
	 */
	SLATEIM_API void BeginBorder(const FSlateBrush* BorderBrush, const FBorderParams& Params = FBorderParams());

	/**
	 * Begin a container with a background image.
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param BorderStyleName The name of the brush in FAppStyle to use as the background for this container
	 * @param Orientation Which direction the contents of the container should flow
	 * @param bAbsorbMouse (optional) Whether the container should handle all mouse inputs
	 * @param ContentPadding (optional) How much to pad the contents of the container
	 */
	SLATEIM_API void BeginBorder(const FName BorderStyleName, EOrientation Orientation, bool bAbsorbMouse = true, FMargin ContentPadding = FMargin(2.0f));

	/**
	 * Begin a container with a background image.
	 *
	 * @param BorderStyleName The name of the brush in FAppStyle to use as the background for this container
	 * @param Params (optional) Additional params to create the border
	 */
	SLATEIM_API void BeginBorder(const FName BorderStyleName, const FBorderParams& Params = FBorderParams());

	SLATEIM_API void EndBorder();

	/**
	 * Begins a container that will allow the user to scroll when its content is larger than the allotted space.
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param Orientation Which direction content should flow and scroll
	 * 
	 * @return Whether the user scroll the content this frame
	 */
	SLATEIM_API bool BeginScrollBox(EOrientation Orientation);

	/**
	 * Begins a container that will allow the user to scroll when its content is larger than the allotted space.
	 *
	 * @param Params (optional) Additional parameters to create the scroll box
	 *
	 * @return Whether the user scroll the content this frame
	 */
	SLATEIM_API bool BeginScrollBox(const FScrollBoxParams& Params = FScrollBoxParams());

	SLATEIM_API void EndScrollBox();

	/**
	 * Begin a container with a background image.
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param BorderStyleName The name of the brush in FAppStyle to use as the background for this container
	 * @param Orientation Which direction the contents of the container should flow
	 * @param bAbsorbMouse (optional) Whether the container should handle all mouse inputs
	 * @param ContentPadding (optional) How much to pad the contents of the container
	 */
	SLATEIM_API void BeginPopUp(const FName BorderStyleName, EOrientation Orientation, bool bAbsorbMouse = true, FMargin ContentPadding = FMargin(2.0f));
	
	/**
	 * Begin a container with a background image.
	 *
	 * @param BorderStyleName The name of the brush in FAppStyle to use as the background for this container
	 * @param Params (optional) Additional parameters to create the border
	 */
	SLATEIM_API void BeginPopUp(const FName BorderStyleName = TEXT("ToolPanel.GroupBorder"), const FPopUpParams& Params = FPopUpParams());

	/**
	 * Begin a container with a background image.
	 *
	 * Please use the method below with the struct-parameter.
	 *
	 * @param BorderBrush The brush to use as the background for this container
	 * @param Orientation Which direction the contents of the container should flow
	 * @param bAbsorbMouse (optional) Whether the container should handle all mouse inputs
	 * @param ContentPadding (optional) How much to pad the contents of the container
	 */
	SLATEIM_API void BeginPopUp(const FSlateBrush* BorderBrush, EOrientation Orientation, bool bAbsorbMouse = true, FMargin ContentPadding = FMargin(2.0f));

	/**
	 * Begin a container with a background image.
	 *
	 * @param BorderBrush The brush to use as the background for this container
	 * @param Params (optional) Additional params to create the border
	 */
	SLATEIM_API void BeginPopUp(const FSlateBrush* BorderBrush, const FPopUpParams& Params = FPopUpParams());
	
	SLATEIM_API void EndPopUp();
#pragma endregion Containers

#pragma region Slots
	/**
	 * Sets the padding for the next widget
	 * 
	 * @param NextPadding The padding to apply to the next widget
	 */
	SLATEIM_API void Padding(const FMargin NextPadding);
	
	/**
	 * Sets the horizontal alignment for the next widget
	 * 
	 * @param NextAlignment The alignment to apply to the next widget
	 */
	SLATEIM_API void HAlign(EHorizontalAlignment NextAlignment);
	/**
	 * Sets the vertical alignment for the next widget
	 * 
	 * @param NextAlignment The alignment to apply to the next widget
	 */
	SLATEIM_API void VAlign(EVerticalAlignment NextAlignment);

	/**
	 * Set the next slot to AutoSize to its content
	 */
	SLATEIM_API void AutoSize();

	/**
	 * Sets the next slot to Fill the remaining space in its container
	 *
	 * @note The filled space is shared equally with any other slots set to Fill the same container
	 */
	SLATEIM_API void Fill();

	/**
	 * Set the minimum width the next slot can have.
	 * 
	 * @param InMinWidth The minimum width for the next slot
	 */
	SLATEIM_API void MinWidth(float InMinWidth);

	/**
	 * Set the minimum height the next slot can have.
	 * 
	 * @param InMinHeight The minimum height for the next slot
	 */

	SLATEIM_API void MinHeight(float InMinHeight);
	/**
	 * Set the maximum width the next slot can have.
	 * 
	 * @param InMaxWidth The maximum width for the next slot
	 */

	SLATEIM_API void MaxWidth(float InMaxWidth);
	/**
	 * Set the maximum height the next slot can have.
	 * 
	 * @param InMaxHeight The maximum height for the next slot
	 */
	SLATEIM_API void MaxHeight(float InMaxHeight);

	/**
	 * Set the alignments to Fill and removes autosize.
	 */
	SLATEIM_API void Maximize();
#pragma endregion Slots

#pragma region Widgets
	/**
	 * Display a string of text
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InText The text string to display
	 * @param TextStyle A Text Block Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *					provide a new style object to update the visuals at runtime
	 */
	SLATEIM_API void Text(const FStringView& InText, const FTextBlockStyle* TextStyle);

	/**
	 * Display a colored string of text
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InText The text string to display
	 * @param Color The color to display the text as
	 * @param TextStyle (optional) A Text Block Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *					provide a new style object to update the visuals at runtime
	 */
	SLATEIM_API void Text(const FStringView& InText, FSlateColor Color, const FTextBlockStyle* TextStyle = nullptr);

	/**
	 * Display a string of text
	 *
	 * @param InText The text string to display
	 * @param Params (optional) Additional parameters to create the text.
	 */
	SLATEIM_API void Text(const FStringView& InText, const FTextParams& Params = FTextParams());

	/**
	 * Create a text input field
	 * 
	 * @param InOutText The text that is in the input field. Can have a default value
	 * @param HintText The hint text to display when the input field is empty
	 * @param TextStyle (optional) An Editable Text Box Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *					provide a new style object to update the visuals at runtime
	 *					
	 * @return Whether the user changed or committed text this frame
	 */
	SLATEIM_API bool EditableText(FString& InOutText, const FStringView& HintText, const FEditableTextBoxStyle* TextStyle = nullptr);

	/**
	 * Create a text input field
	 *
	 * @param InOutText The text that is in the input field. Can have a default value
	 * @param Params (optional) Additional parameters to create the widget
	 *
	 * @return Whether the user changed or committed text this frame
	 */
	SLATEIM_API bool EditableText(FString& InOutText, const FEditableTextParams& Params = FEditableTextParams());

	/**
	 * Display a brush
	 * 
	 * @param ImageBrush The brush to display
	 * @param ColorAndOpacity The color to tint the brush
	 * @param DesiredSize (optional) Override the desired size to display the brush at
	 */
	SLATEIM_API void Image(const FSlateBrush* ImageBrush, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize = FVector2D::ZeroVector);

	/**
	 * Display a brush
	 *
	 * @param ImageBrush The brush to display
	 * @param Params (optional) Additional parameters to display the image
	 */
	SLATEIM_API void Image(const FSlateBrush* ImageBrush, const FImageParams& Params = FImageParams());

	/**
	 * Display a slate style brush
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ImageStyleName The named of the brush to display from FAppStyle
	 * @param ColorAndOpacity The color to tint the brush
	 * @param DesiredSize (optional) Override the desired size to display the brush at
	 */
	SLATEIM_API void Image(const FName ImageStyleName, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize = FVector2D::ZeroVector);
	
	/**
	 * Display a slate style brush
	 *
	 * @param ImageStyleName The named of the brush to display from FAppStyle
	 * @param Params (optional) Additional parameters to display the image
	 */
	SLATEIM_API void Image(const FName ImageStyleName, const FImageParams& Params = FImageParams());

	/**
	 * Display a colored box
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ColorAndOpacity The color to display
	 * @param DesiredSize (optional) Override the desired size to display the box at
	 */
	SLATEIM_API void Image(const FSlateColor& ColorAndOpacity, FVector2D DesiredSize = FVector2D::ZeroVector);

	/**
	 * Display a colored box
	 *
	 * @param Params (optional) Additional parameters to display the block (including the color)
	 */
	SLATEIM_API void Image(const FImageParams& Params = FImageParams());

#if WITH_ENGINE
	/**
	 * Display a texture
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ImageTexture The texture to display
	 * @param ColorAndOpacity (optional) The color to tint the brush
	 * @param DesiredSize (optional) Override the desired size to display the brush at
	 */
	
	SLATEIM_API void Image(UTexture2D* ImageTexture, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize = FVector2D::ZeroVector);
	
	/**
	 * Display a texture
	 *
	 * @param ImageTexture The texture to display
	 * @param Params (optional) Additional parameters to display the image
	 */
	SLATEIM_API void Image(UTexture2D* ImageTexture, const FImageParams& Params = FImageParams());

	/**
	 * Display a render target texture
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ImageRenderTarget The render target texture to display
	 * @param ColorAndOpacity The color to tint the brush
	 * @param DesiredSize (optional) Override the desired size to display the brush at
	 */
	
	SLATEIM_API void Image(UTextureRenderTarget2D* ImageRenderTarget, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize = FVector2D::ZeroVector);
	
	/**
	 * Display a render target texture
	 *
	 * @param ImageRenderTarget The render target texture to display
	 * @param Params (optional) Additional parameters to display the image
	 */
	SLATEIM_API void Image(UTextureRenderTarget2D* ImageRenderTarget, const FImageParams& Params = FImageParams());

	/**
	 * Display a material
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ImageMaterial The material to display
	 * @param BrushSize The size to create the internal brush with
	 * @param ColorAndOpacity The color to tint the brush
	 * @param DesiredSize (optional) Override the desired size to display the brush at
	 */
	SLATEIM_API void Image(UMaterialInterface* ImageMaterial, FVector2D BrushSize, const FSlateColor& ColorAndOpacity, FVector2D DesiredSize = FVector2D::ZeroVector);

	/**
	 * Display a material
	 *
	 * @param ImageMaterial The material to display
	 * @param BrushSize The size to create the internal brush with
	 * @param Params (optional) Additional parameters to display the image
	 */
	SLATEIM_API void Image(UMaterialInterface* ImageMaterial, FVector2D BrushSize, const FImageParams& Params = FImageParams());
#endif

	/**
	 * Display a button with text
	 * 
	 * Please use the method below with the struct-parameter.

	 * @param InText The text to display on the button
	 * @param bEnabled Indicate if the button is enabled or not (grayed)
	 * @param InStyle (optional) A Button Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *				  provide a new style object to update the visuals at runtime
	 *					
	 * @return Whether the user clicked the button this frame
	 */
	SLATEIM_API bool Button(const FStringView& InText, const bool bEnabled, const FButtonStyle* InStyle = nullptr);

	/**
	 *
	 * @param InText The text to display on the button
	 * @param Params (optional) Additional parameters to create the button
	 *
	 * @return Whether the user clicked the button this frame
	 */
	SLATEIM_API bool Button(const FStringView& InText, const FButtonParams& Params = FButtonParams());

	/**
	 * Display a two-state checkbox
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InLabel The label for the checkbox
	 * @param InOutCurrentState The current state of the checkbox. Can provide a default value.
	 * @param CheckBoxStyle (optional) A Checkbox Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *						provide a new style object to update the visuals at runtime
	 *					
	 * @return Whether the user changed the state of the checkbox this frame
	 */
	SLATEIM_API bool CheckBox(const FStringView& InLabel, bool& InOutCurrentState, const FCheckBoxStyle* CheckBoxStyle = nullptr);

	/**
	 * Display a two-state checkbox
	 *
	 * @param InOutCurrentState The current state of the checkbox. Can provide a default value.
	 * @param Params (optional) Additional parameters to create the checkbox
	 *
	 * @return Whether the user changed the state of the checkbox this frame
	 */
	SLATEIM_API bool CheckBox(bool& InOutCurrentState, const FCheckBoxParams& Params = FCheckBoxParams());
	
	/**
	 * Display a three-state checkbox
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InText The label for the checkbox
	 * @param InOutCurrentState The current state of the checkbox. Can provide a default value.
	 * @param CheckBoxStyle (optional) A Checkbox Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *						provide a new style object to update the visuals at runtime
	 *					
	 * @return Whether the user changed the state of the checkbox this frame
	 */
	SLATEIM_API bool CheckBox(const FStringView& InText, ECheckBoxState& InOutCurrentState, const FCheckBoxStyle* CheckBoxStyle = nullptr);

	/**
	 * Display a three-state checkbox
	 *
	 * @param InOutCurrentState The current state of the checkbox. Can provide a default value.
	 * @param Params (optional) Additional parameters to create the checkbox
	 *
	 * @return Whether the user changed the state of the checkbox this frame
	 */
	SLATEIM_API bool CheckBox(ECheckBoxState& InOutCurrentState, const FCheckBoxParams& Params = FCheckBoxParams());

	/**
	 * Display a float-based spin box
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Min The minimum value of the spin box (or unset for no limit).
	 * @param Max The maximum value of the spin box (or unset for no limit).
	 * @param SpinBoxStyle (optional) A Spin Box Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *						provide a new style object to update the visuals at runtime
	 * 
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	SLATEIM_API bool SpinBox(float& InOutValue, TOptional<float> Min, TOptional<float> Max, const FSpinBoxStyle* SpinBoxStyle = nullptr);

	/**
	 * Display a double-based spin box
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Min The minimum value of the spin box (or unset for no limit).
	 * @param Max The maximum value of the spin box (or unset for no limit).
	 * @param SpinBoxStyle (optional) A Spin Box Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *						provide a new style object to update the visuals at runtime
	 * 
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	SLATEIM_API bool SpinBox(double& InOutValue, TOptional<double> Min, TOptional<double> Max, const FSpinBoxStyle* SpinBoxStyle = nullptr);

	/**
	 * Display an integer-based spin box
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Min The minimum value of the spin box (or unset for no limit).
	 * @param Max The maximum value of the spin box (or unset for no limit).
	 * @param SpinBoxStyle (optional) A Spin Box Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *						provide a new style object to update the visuals at runtime
	 * 
	 * @return Whether the user committed the spin box value this frame
	 */
	SLATEIM_API bool SpinBox(int32& InOutValue, TOptional<int32> Min, TOptional<int32> Max, const FSpinBoxStyle* SpinBoxStyle = nullptr);

	/**
	 * Display a spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params (optional) Additional parameters to create the spinbox
	 *
	 * @return Whether the user committed the spin box value this frame
	 */
	template<typename InNumericType>
	bool SpinBox(InNumericType& InOutValue, const FSpinBoxParams<InNumericType>& Params = FSpinBoxParams<InNumericType>());

	/**
	 * Display a float-based spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params Additional parameters to create the spinbox
	 * 
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	template<>
	bool SLATEIM_API SpinBox(float& InOutValue, const FSpinBoxParams<float>& Params);

	/**
	 * Display a double-based spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params Additional parameters to create the spinbox
	 *
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	template<>
	bool SLATEIM_API SpinBox(double& InOutValue, const FSpinBoxParams<double>& Params);

	/**
	 * Display an int32-based spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params Additional parameters to create the spinbox
	 * 
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	template<>
	bool SLATEIM_API SpinBox(int32& InOutValue, const FSpinBoxParams<int32>& Params);

	/**
	 * Display a float-based slider
	 *
	 * Please use the method below with the struct-parameter.
	 *
	 * @param InOutValue The current value of the slider. Can provide a default value.
	 * @param Min The minimum value of the slider
	 * @param Max The maximum value of the slider
	 * @param Step The smallest incremental change that can be made to the slider's value
	 * @param SliderStyle (optional) A Slider Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *					  provide a new style object to update the visuals at runtime
	 *
	 * @return Whether the user changed the value of the slider this frame
	 */
	SLATEIM_API bool Slider(float& InOutValue, float Min, float Max, float Step, const FSliderStyle* SliderStyle = nullptr);

	/**
	 * Display a float-based slider
	 *
	 * @param InOutValue The current value of the slider. Can provide a default value.
	 * @param Params (optional) Additional parameters to create the slider
	 *
	 * @return Whether the user changed the value of the slider this frame
	 */
	SLATEIM_API bool Slider(float& InOutValue, const FSliderParams& Params = FSliderParams());

	/**
	 * Display a progress bar
	 * 
	 * @param Percent The value of the progress bar (0.0 to 1.0)
	 * @param ProgressBarStyle A Progress Bar Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *						   provide a new style object to update the visuals at runtime
	 */
	SLATEIM_API void ProgressBar(TOptional<float> Percent, const FProgressBarStyle* ProgressBarStyle);

	/**
	 * Display a progress bar
	 *
	 * @param Percent The value of the progress bar (0.0 to 1.0)
	 * @param Params (optional) Additional parameters to create the progress bar
	 */
	SLATEIM_API void ProgressBar(TOptional<float> Percent, const FProgressBarParams& Params = FProgressBarParams());

	/**
	 * Display a dropdown of text options
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ComboItems The options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param bForceRefresh (optional) Whether to force a refresh of the available options or the selected option (set to true for a frame when changing the list of options or manually setting the selected index)
	 * @param InComboStyle (optional) A Combo Box Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *					   provide a new style object to update the visuals at runtime
	 *					   
	 * @return Whether the user changed the selected option this frame
	 */
	SLATEIM_API bool ComboBox(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, bool bForceRefresh, const FComboBoxStyle* InComboStyle = nullptr);

	/**
	 * Display a searchable dropdown of text options
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ComboItems The options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param bForceRefresh (optional) Whether to force a refresh of the available options or the selected option (set to true for a frame when changing the list of options or manually setting the selected index)
	 * @param InComboStyle (optional) A Combo Box Style to override the default style. Only the initial style is applied
	 *					   
	 * @return Whether the user changed the selected option this frame
	 */
	SLATEIM_API bool SearchableComboBox(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, bool bForceRefresh = false, const FComboBoxStyle* InComboStyle = nullptr);

	/**
	 * Display a dropdown of text options
	 *
	 * @param ComboItems The options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param Params (optional) Additional parameter to create the combo box
	 *
	 * @return Whether the user changed the selected option this frame
	 */
	SLATEIM_API bool ComboBox(const TArray<FString>& ComboItems, int32& InOutSelectedItemIndex, const FComboBoxParams& Params = FComboBoxParams());

	/**
	 * Display a list of text options
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param ListItems The list of options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param bForceRefresh (optional) Whether to force a refresh of the available options or the selected option (set to true for a frame when changing the list of options or manually setting the selected index)
	 * @param InStyle (optional) A Table View Style to override the default style. Changes to the style after the first invocation are not reflected,
	 *				  provide a new style object to update the visuals at runtime
	 *					   
	 * @return Whether the user changed the selected option this frame
	 */
	SLATEIM_API bool SelectionList(const TArray<FString>& ListItems, int32& InOutSelectedItemIndex, bool bForceRefresh, const FTableViewStyle* InStyle = nullptr);

	/**
	 * Display a list of text options
	 *
	 * @param ListItems The list of options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param Params (optional) Additional parameters to create the selection list
	 *
	 * @return Whether the user changed the selected option this frame
	 */
	SLATEIM_API bool SelectionList(const TArray<FString>& ListItems, int32& InOutSelectedItemIndex, const FSelectionListParams& Params = FSelectionListParams());

	/**
	 * Create a block of empty space
	 * 
	 * @param Size The size of whitespace to create
	 */
	SLATEIM_API void Spacer(const FVector2D& Size);

	/**
	 * Add an already created slate widget to the IM hierarchy
	 *
	 * @note Do not create the widget you are passing in here every frame. This is meant to be used to add already created widgets
	 * @note Not available in BPFL: TSharedRef<SWidget> is not Blueprint-compatible
	 */
	SLATEIM_API void Widget(TSharedRef<SWidget> InWidget);
#pragma endregion Widgets

#pragma region Menu
	/**
	 * Begin an area where a menu appears with a right click
	 *
	 * @return Whether the menu is open
	 */
	SLATEIM_API bool BeginContextMenuAnchor();

	SLATEIM_API void EndContextMenuAnchor();

	/** 
	* Begin a menu bar, similar to those shown at the top of windows. Can be placed anywhere.
	* 
	* @see AddMenuSeparator AddMenuButton AddMenuToggleButton AddMenuCheckButton BeginSubMenu
	*/
	SLATEIM_API void BeginMenuBar();

	/**
	 * End the menu bar.
	 */
	SLATEIM_API void EndMenuBar();

	/** 
	 * Adds a menu to the menu entry with the given name.
	 */
	SLATEIM_API void AddMenuBarEntry(const FStringView& InMenuName);

	/**
	 * Ends a menu bar entry 
	 */
	SLATEIM_API void EndMenuBarEntry();

	/**
	 * Add a new section to the current context menu
	 * 
	 * @param SectionText The label of the section to add
	 */
	SLATEIM_API void AddMenuSection(const FStringView& SectionText);
	
	/**
	 * Add a separator to the current context menu
	 */
	SLATEIM_API void AddMenuSeparator();

	/**
	 * Add a menu item button to the current context menu
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param RowText Label for the menu item
	 * @param ToolTipText The tooltip to display for the menu item
	 * 
	 * @return Whether the user clicked the menu item this frame
	 */
	SLATEIM_API bool AddMenuButton(const FStringView& RowText, const FStringView& ToolTipText);

	/**
	 * Add a menu item button to the current context menu
	 *
	 * @param RowText Label for the menu item
	 * @param Params (optional) Additional parameters to create the menu button
	 *
	 * @return Whether the user clicked the menu item this frame
	 */
	SLATEIM_API bool AddMenuButton(const FStringView& RowText, const FMenuButtonParams& Params = FMenuButtonParams());

	/**
	 * Display a menu item button with a checkbox
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param RowText Label for the menu item
	 * @param InOutCurrentState The current state of the toggle
	 * @param ToolTipText The tooltip to display for the menu item
	 * 
	 * @return Whether the user toggled the checkbox this frame
	 */
	SLATEIM_API bool AddMenuToggleButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText);

	/**
	 * Display a menu item button with a checkbox
	 *
	 * @param RowText Label for the menu item
	 * @param InOutCurrentState The current state of the toggle
	 * @param Params (optional) Additional parameters to create the menu button
	 *
	 * @return Whether the user toggled the checkbox this frame
	 */
	SLATEIM_API bool AddMenuToggleButton(const FStringView& RowText, bool& InOutCurrentState, const FMenuButtonParams& Params = FMenuButtonParams());

	/**
	 * Display a menu item button with a checkmark
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param RowText Label for the menu item
	 * @param InOutCurrentState The current state of the checkmark
	 * @param ToolTipText The tooltip to display for the menu item
	 * 
	 * @return Whether the user clicked the menu item this frame
	 */
	SLATEIM_API bool AddMenuCheckButton(const FStringView& RowText, bool& InOutCurrentState, const FStringView& ToolTipText);

	/**
	 * Display a menu item button with a checkmark
	 *
	 * @param RowText Label for the menu item
	 * @param InOutCurrentState The current state of the checkmark
	 * @param Params (optional) Additional parameters to create the menu button
	 *
	 * @return Whether the user clicked the menu item this frame
	 */
	SLATEIM_API bool AddMenuCheckButton(const FStringView& RowText, bool& InOutCurrentState, const FMenuButtonParams& Params = FMenuButtonParams());

	/**
	 * Add a submenu item to the current menu
	 * 
	 * @param SubMenuText The label for the menu item
	 */
	SLATEIM_API void BeginSubMenu(const FStringView& SubMenuText);

	SLATEIM_API void EndSubMenu();
#pragma endregion Menu

#pragma region Tabs
	/**
	 * Begin a tab group, can contain TabStacks and TabSplitters
	 * 
	 * @param TabGroupId Provide a unique identifier for this tab group
	 */
	SLATEIM_API void BeginTabGroup(const FName& TabGroupId);
	
	/**
	 * Ends the current Tab Group
	 */
	SLATEIM_API void EndTabGroup();

	/**
	 * Begin a tab stack, can only contain Tabs
	 */
	SLATEIM_API void BeginTabStack();

	/**
	 * Ends the current Tab Stack
	 */
	SLATEIM_API void EndTabStack();

	/**
	 * Begin a tab splitter, displays child TabSplitters and TabStacks side-by-side
	 * 
	 * @param Orientation The direction to layout child TabSplitters and TabStacks
	 */
	SLATEIM_API void BeginTabSplitter(EOrientation Orientation);

	/**
	 * Ends the current Tab Splitter
	 */
	SLATEIM_API void EndTabSplitter();

	/**
	 * Assigns the Size Coefficient for the next child of a TabSplitter
	 * 
	 * @param SizeCoefficient The weight to assign to the next child of the parent TabSplitter
	 */
	SLATEIM_API void TabSplitterSizeCoefficient(float SizeCoefficient);

	/**
	 * Begins a Tab to contain any other content
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param TabId Provide a tab id that is unique within the Tab Group
	 * @param TabIcon An icon to display on the tab next to the title. This cannot be updated after the initial creation of the tab.
	 * @param TabTitle (optional) The title to display for the tab if it should differ from the TabId. This cannot be updated after the initial creation of the tab.
	 * 
	 * @return Returns true if the tab is active, false otherwise. Logic to draw the contents of the tab can be skipped when this is false.
	 */
	SLATEIM_API bool BeginTab(const FName& TabId, const FSlateIcon& TabIcon, const FText& TabTitle = FText::GetEmpty());

	/**
	 * Begins a Tab to contain any other content
	 *
	 * @param TabId Provide a tab id that is unique within the Tab Group
	 * @param Params (optional) Additional parameters used to create the tab
	 *
	 * @return Returns true if the tab is active, false otherwise. Logic to draw the contents of the tab can be skipped when this is false.
	 */
	SLATEIM_API bool BeginTab(const FName& TabId, const FTabParams& Params = FTabParams());

	/**
	 * Ends the current Tab. This must be called even when BeginTab returns false.
	 */
	SLATEIM_API void EndTab();

	/**
	 * Causes a one-time activation of a tab in its parent tab well. Can be used at any point during layout.
	 * 
	 * To use this effectively, all tab ids should be unique. Only the first tab registered with the given id will be activated.
	 * Can be called multiple times to activate multiple tabs in the same update.
	 * @see ActivateInParent(ETabActivationCause).
	 * 
	 * @param TabId The tab id to activate.
	 */
	SLATEIM_API void ActivateTab(const FName& TabId);
#pragma endregion Tabs

#pragma region Util
	/**
	 * SlateIM updates are disabled in specific scenarios (like when a SlateIM::ModalDialog is open), use this function to react accordingly
	 * 
	 * @return Whether SlateIM can update currently
	 */
	SLATEIM_API bool CanUpdateSlateIM();
	
	/**
	 * Disables all widgets until EndDisabledState is called.
	 * 
	 * @note It is not possible to enable a widget inside a disabled parent widget by calling EndDisabledState before a child is created inside a disabled Widget.
	 */
	SLATEIM_API void BeginDisabledState();

	SLATEIM_API void EndDisabledState();

	/**
	 * Sets the tooltip to be used for the next widget created. Resets after tooltip is used
	 *
	 * @param NextToolTip The tooltip to display
	 */
	SLATEIM_API void SetToolTip(const FStringView& NextToolTip);

	/**
	 * Opens a modal dialog of the specified type. This function will not return until the user closes the modal dialog
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param MessageType The type of options the user can respond to the dialog with
	 * @param DialogText The message to display to the user
	 * @param Category The type of dialog to display
	 * @param DialogTitle (optional) The title to display in the dialog window
	 * 
	 * @return The option the user selected
	 *
	 * @note SlateIM updates are disable while a SlateIM modal is open
	 * @see CanUpdateSlateIM()
	 */
	SLATEIM_API EAppReturnType::Type ModalDialog(EAppMsgType::Type MessageType, const FStringView& DialogText, EAppMsgCategory Category, const FStringView& DialogTitle = FStringView());

	/**
	 * Opens a modal dialog of the specified type. This function will not return until the user closes the modal dialog
	 *
	 * @param MessageType The type of options the user can respond to the dialog with
	 * @param DialogText The message to display to the user
	 * @param Params (optional) Additional parameters used to create the dialog
	 *
	 * @return The option the user selected
	 *
	 * @note SlateIM updates are disable while a SlateIM modal is open
	 * @see CanUpdateSlateIM()
	 */
	SLATEIM_API EAppReturnType::Type ModalDialog(EAppMsgType::Type MessageType, const FStringView& DialogText, const FModalDialogParams& Params = FModalDialogParams());

	/**
	 * Text filter input box that updates the filter when text changes.
	 *
	 * @param InFilter The text filter to display and update
	 * @param Params (optional) Additional parameters to create the text filter
	 *
	 * @return Whether the user changed or committed text this frame
	 *
	 * @note Not available in BPFL: template function
	 */
	template<typename ItemType>
	bool TextFilter(FSlateImTextFilter<ItemType>& InFilter, const FEditableTextParams& Params = FEditableTextParams())
	{
		if (EditableText(InFilter.InputText, Params) && InFilter.TextFilter.IsValid())
		{
			InFilter.TextFilter->SetRawFilterText(FText::FromString(InFilter.InputText));
			return true;
		}
		return false;
	}

	/**
	 * Text filter input box that updates the filter when text changes.
	 *
	 * Please use the method above with the struct-parameter.
	 *
	 * @param InFilter The text filter to display and update
	 * @param HintText hint text to display when the input is empty
	 * 
	 * @return Whether the user changed or committed text this frame
	 *
	 * @note Not available in BPFL: template function
	 */
	template<typename ItemType>
	bool TextFilter(FSlateImTextFilter<ItemType>& InFilter, const FStringView& HintText)
	{
		return TextFilter<ItemType>(InFilter, {.HintText = HintText});
	}
#pragma endregion Util

#pragma region Queries
	/**
	 * Query whether the last widget is hovered or not
	 * 
	 * @return true if the cursor is hovering the last rendered widget, false otherwise
	 */
	SLATEIM_API bool IsHovered();

	/**
	 * Query whether the previous widget is focused
	 * 
	 * @param Depth (optional) How far to check for focus. Includes child widgets by default.
	 * @return true if the previous widget has focus (for the specified depth), false otherwise
	 */
	SLATEIM_API bool IsFocused(EFocusDepth Depth = EFocusDepth::IncludingDescendants);
#pragma endregion Queries

#pragma region Graphing
	/**
	 * Begin a graph widget, call graphing functions between this and EndGraph to include multiple graphs in a single chart
	 */
	SLATEIM_API void BeginGraph();

	/**
	 * Call when finished calling graphing functions for the widget
	 */
	SLATEIM_API void EndGraph();

	/**
	 * Add a line graph of 2D vectors to the graph widget
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param Points The X,Y points to plot on the graph
	 * @param LineColor The color of the line to draw for this graph
	 * @param LineThickness How thick to draw the line
	 * @param XViewRange The min and max X values to horizontally scale the graph to
	 * @param YViewRange The min and max Y values to vertically scale the graph to
	 */
	SLATEIM_API void GraphLine(const TArrayView<FVector2D>& Points, const FLinearColor& LineColor, float LineThickness, const FDoubleRange& XViewRange, const FDoubleRange& YViewRange);

	/**
	 * Add a line graph of 2D vectors to the graph widget
	 *
	 * @param Points The X,Y points to plot on the graph
	 * @param Params Additional parameters to draw the line
	 */
	SLATEIM_API void GraphLine(const TArrayView<FVector2D>& Points, const FGraphLinePointsParams& Params = FGraphLinePointsParams());

	/**
	 * Add a line graph of values to the graph widget.
	 * The value index is used as the X-value when plotting, the graph will scale horizontally to fit all values in the array.
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param Values The Y values to plot on the graph
	 * @param LineColor The color of the line to draw for this graph
	 * @param LineThickness How thick to draw the line
	 * @param ViewRange The min and max Y values to vertically scale the graph to
	 */
	SLATEIM_API void GraphLine(const TArrayView<double>& Values, const FLinearColor& LineColor, float LineThickness, const FDoubleRange& ViewRange);

	/**
	 * Add a line graph of values to the graph widget.
	 * The value index is used as the X-value when plotting, the graph will scale horizontally to fit all values in the array.
	 *
	 * @param Values The Y values to plot on the graph
	 * @param Params Additional parameters to draw the line
	 */
	SLATEIM_API void GraphLine(const TArrayView<double>& Values, const FGraphLineValuesParams& Params = FGraphLineValuesParams());
#pragma endregion Graphing

#pragma region Inputs
	/**
	 * Query whether a key was pressed this frame.
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 * 
	 * @param InKey The key to query the state of
	 * @return true if the key was just pressed this frame
	 */
	SLATEIM_API bool IsKeyPressed(const FKey& InKey);
	
	/**
	 * Query whether a key is being held this frame.
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 * 
	 * @param InKey The key to query the state of
	 * @return true if the key was pressed or held this frame
	 */
	SLATEIM_API bool IsKeyHeld(const FKey& InKey);
	
	/**
	 * Query whether a key was released this frame.
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 * 
	 * @param InKey The key to query the state of
	 * @return true if the key was just released this frame
	 */
	SLATEIM_API bool IsKeyReleased(const FKey& InKey);
	
	/**
	 * Retrieve the analog value for a key
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 * 
	 * @param InKey The key to query the state of
	 * @return the last analog value the SlateIM root received for the specified key
	 */
	SLATEIM_API float GetKeyAnalogValue(const FKey& InKey);
#pragma endregion Inputs

#pragma region Canvas
#if WITH_ENGINE
	namespace Canvas
	{
		/**
		 * Begin queueing commands to draw to a canvas render target.
		 * 
		 * @param Width The width of the canvas in pixels.
		 * @param Height The height of the canvas in pixels.
		 * @param Params Additional parameters to create the canvas.
		 */
		SLATEIM_API void BeginCanvas(int32 Width, int32 Height, const FCanvasParams& CanvasParams = FCanvasParams());

		/**
		 * Ends the currently drawn canvas and executes all queued commands.
		 */
		SLATEIM_API void EndCanvas();

		/**
		 * Invalidate the current canvas, forcing a redraw if its UpdateType is set to Invalidation.
		 */
		SLATEIM_API void Invalidate();

		/** Sets the position of the lower-right corner of the clipping region of the canvas */
		SLATEIM_API void SetClip(const FVector2f& ClipPosition);

		/** Sets the draw color of the canvas. */
		SLATEIM_API void SetDrawColor(const FLinearColor& Color);
		SLATEIM_API void SetDrawColor(const FColor& Color);

		/**
		 * Queue drawing a canvas item to the canvas.
		 *
		 * @param Item The canvas item to draw. The item *must* stay in scope until EndCanvas().
		 * @param CanvasPosition (optional) The position to draw the item on the canvas.
		 *
		 * @note Not available in BPFL: TSharedRef<FCanvasItem> is not Blueprint-compatible
		 */
		SLATEIM_API void DrawItem(TSharedRef<FCanvasItem> ItemPtr, const FVector2f& CanvasPosition = FVector2f::ZeroVector);

		/**
		 * Queue drawing a canvas icon (texture) to the canvas.
		 * 
		 * @param Icon The icon to draw. The texture draw area is in pixels.
		 * @param CanvasPosition (optional) The position of the icon on the canvas, in pixels.
		 * @param Scale (optional) The scale of the icon.
		 */
		SLATEIM_API void DrawIcon(const FCanvasIcon& Icon, const FVector2f& CanvasPosition = FVector2f::ZeroVector, const FVector2f& Scale = FVector2f::UnitVector);

		/**
		 * Queue drawing a line to the canvas.
		 * 
		 * @param CanvasPositionA One end of the line on the canvas, in pixels.
		 * @param CanvasPositionB The other end of the line on the canvas, in pixels.
		 * @param Thickness (optional) The thickness of the line in pixels.
		 * @param RenderColor (optional) The color of the line.
		 */
		SLATEIM_API void DrawLine(const FVector2D& CanvasPositionA, const FVector2D& CanvasPositionB, float Thickness = 1.0f, const FLinearColor& RenderColor = FLinearColor::White);

		/**
		 * Queue drawing a texture to the canvas.
		 *
		 * @param RenderTexture The texture to draw.
		 * @param CanvasPosition The position of the texture on the canvas, in pixels.
		 * @param CanvasSize The size of the texture on the canvas, in pixels.
		 * @param RenderParams (optional) Additional parameters to draw the tile.
		 */
		SLATEIM_API void DrawTexture(UTexture* RenderTexture, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FTileRenderParams& RenderParams = FTileRenderParams());

		/**
		 * Queue drawing a material to the canvas.
		 *
		 * @param RenderMaterial The material to draw.
		 * @param CanvasPosition The position of the material on the canvas, in pixels.
		 * @param CanvasSize The size of the material on the canvas, in pixels.
		 * @param RenderParams (optional) Additional parameters to draw the tile. RenderColor and BlendMode are not used.
		 */
		SLATEIM_API void DrawMaterial(UMaterialInterface* RenderMaterial, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FTileRenderParams& RenderParams = FTileRenderParams());

		/**
		 * Queue drawing text to the canvas.
		 *
		 * @param RenderFont The font used to render the text.
		 * @param RenderText The string to render.
		 * @param CanvasPosition The position of the text on the canvas, in pixels.
		 * @param RenderParams (optional) Additional parameters to draw the text.
		 */
		SLATEIM_API void DrawText(UFont* RenderFont, const FString& RenderText, const FVector2D& CanvasPosition, const FTextRenderParams& RenderParams);

		/**
		 * Queue drawing text to the canvas.
		 * 
		 * Alternate method allowing the use of FFontRenderInfo.
		 *
		 * @param RenderFont The font used to render the text.
		 * @param RenderText The string to render.
		 * @param CanvasPosition The position of the text on the canvas, in pixels.
		 * @param Scale (optional) The scale of the text on the canvas.
		 * @param RenderInfo (optional) Extra optional parameters for rendering the text.
		 */
		SLATEIM_API void DrawText(UFont* RenderFont, const FString& RenderText, const FVector2f& CanvasPosition, const FVector2f& Scale = FVector2f::UnitVector, const FFontRenderInfo& RenderInfo = FFontRenderInfo());

		/**
		 * Queue drawing wrapped text to canvas.
		 * 
		 * Uses FFontRenderInfo for text effets.
		 * 
		 * Text size / wrapping is automatically computed based on the position of the text and the @see SetCanvasClip.
		 * The default clipping position is the entire canvas. It does not reset between queued calls.
		 * 
		 * @param RenderFont The font used to render the text.
		 * @param RenderText The text to render.
		 * @param CanvasPosition The position of the text on the canvas, in pixels.
		 * @param bCenterTextX If true, the text is centered on the given draw position on the X axis.
		 * @param bCenterTextY If true, the text is centered on the given draw position on the Y axis.
		 * @param Scale (optional) The scale of the text on the canvas.
		 * @param RenderInfo (optional) Extra optional parameters for rendering the text.
		 */
		SLATEIM_API void DrawWrappedText(const UFont* RenderFont, const FString& RenderText, const FVector2f& CanvasPosition, bool bCenterTextX, bool bCenterTextY, const FVector2f& Scale = FVector2f::UnitVector, const FFontRenderInfo& RenderInfo = FFontRenderInfo());

		/**
		 * Queue drawing a border to the canvas.
		 * 
		 * @param BorderTexture The texture for the center of the border.
		 * @param BackgroundTexture The texture for the background of the border.
		 * @param LeftBorderTexture The texture for the left of the border.
		 * @param RightBorderTexture The texture for the right of the border.
		 * @param TopBorderTexture The texture for the top of the border.
		 * @param BottomBorderTexture The texture for the bottom of the border.
		 * @param CanvasPosition The position of the border on the canvas, in pixels.
		 * @param CanvasSize The size of the border on the canvas, in pixels.
		 * @param RenderParams (optional) Additional parameters to draw the border.
		 */
		SLATEIM_API void DrawBorder(UTexture* BorderTexture, UTexture* BackgroundTexture, UTexture* LeftBorderTexture, UTexture* RightBorderTexture, UTexture* TopBorderTexture, UTexture* BottomBorderTexture, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FBorderRenderParams& RenderParams = FBorderRenderParams());

		/**
		 * Queue drawing an unfilled box to the canvas.
		 * 
		 * @param CanvasPosition The position of the box on the canvas, in pixels.
		 * @param CanvasSize The size of the box on the canvas, in pixels.
		 * @param Thickness (optional) The thicknses of the lines to draw.
		 * @param RenderColor (optional) The color of the lines to draw.
		 */
		SLATEIM_API void DrawBox(const FVector2D& CanvasPosition, const FVector2D& CanvasSize, float Thickness = 1.0f, FLinearColor RenderColor = FLinearColor::White);

		/**
		 * Queue drawing a series of texture-filled triangles to the canvas.
		 *
		 * @param RenderTexture The texture to draw with.
		 * @param Triangles The list of triangles to draw.
		 *
		 * @note Not available in BPFL: TSharedRef<TArray<FCanvasUVTri>> is not Blueprint-compatible
		 */
		SLATEIM_API void DrawTriangles(UTexture* RenderTexture, TSharedRef<TArray<FCanvasUVTri>> Triangles);

		/**
		 * Queue drawing a series of material-filled triangles to the canvas.
		 *
		 * @param RenderTexture The texture to draw with.
		 * @param Triangles The list of triangles to draw.
		 *
		 * @note Not available in BPFL: TSharedRef<TArray<FCanvasUVTri>> is not Blueprint-compatible
		 */
		SLATEIM_API void DrawMaterialTriangles(UMaterialInterface* RenderMaterial, TSharedRef<TArray<FCanvasUVTri>> Triangles);

		/**
		 * Queue drawing a filled polygon to the canvas.
		 * 
		 * @param RenderTexture The texture to draw with.
		 * @param CanvasPosition The position of the center of the polygon on the canvas, in pixels.
		 * @param Radius The radius of the polygon, in pixels.
		 * @param NumberOfSides The number of sides of the polygon.
		 * @param RenderColor (optional) The color of the polygon.
		 */
		SLATEIM_API void DrawPolygon(UTexture* RenderTexture, const FVector2D& CanvasPosition, const FVector2D& Radius, int32 NumberOfSides, FLinearColor RenderColor = FLinearColor::White);
	}
#endif // WITH_ENGINE
#pragma endregion Canvas
}

