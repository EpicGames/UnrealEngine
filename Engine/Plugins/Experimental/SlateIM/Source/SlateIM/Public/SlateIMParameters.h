// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateIMParametersFwd.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Layout/Anchors.h"

#if WITH_ENGINE
#include "SlateIMEngineParameters.h"
#endif

#include "SlateIMParameters.generated.h"

namespace SlateIM
{
	// Sync with FSlateIMWindowParams
	struct FWindowParams
	{
		// The name of the window. This will also be displayed in the windows title bar
		FStringView WindowTitle;

		// The size to open the window at (does not update the size of an existing window)
		FVector2f WindowSize = FVector2f::ZeroVector;

		// If this window was created with a previous call to BeginWindow and then closed, passing in true will reopen the window and false will leave it closed. If the window has never been seen this parameter does nothing and the window will open
		bool bShouldReopen = false;

		// Window will be create always on top.
		bool bAlwaysOnTop = false;
	};

	// Sync with FSlateIMTableParams
	struct FTableParams
	{
		// A Table View Style to override the default style. Changes to the style after the first invocation are not reflected,
		// provide a new style object to update the visuals at runtime
		const FTableViewStyle* Style = nullptr;

		// A Table Row Style to override the default table row style. Changing the style only affects new table rows.
		const FTableRowStyle* RowStyle = nullptr;

		// Select mode for the table.
		ESelectionMode::Type SelectionMode = ESelectionMode::None;
	};

	// Sync with FSlateIMTableColumnParams
	struct FTableColumnParams
	{
		/** Added a default label to the column */
		FStringView Label = FStringView();
	};

	// Sync with FSlateIMTableRowChildrenParams
	struct FTableRowChildrenParams
	{
		/** Id of the parent row. 0 to disable state saving. Row ids must be unique to its parent. */
		uint32 ParentRowId = 0;

		/** Set the default expansion state of the parent (whether the children following are visible.) Only applies to rows with a parent id. */
		bool bDefaultExpanded = false;
	};

	// Sync with FSlateIMTextParams
	struct FTextParams
	{
		/** The color to display the text as */
		FSlateColor Color = FSlateColor::UseForeground();

		/** A Text Block Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FTextBlockStyle* Style = nullptr;
	};

	// Sync with FSlateIMEditableTextParams
	struct FEditableTextParams
	{
		/** HintText The hint text to display when the input field is empty */
		const FStringView HintText = FStringView();

		/** An Editable Text Box Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FEditableTextBoxStyle* Style = nullptr;
	};

	// Sync with FSlateIMButtonParams
	struct FButtonParams
	{
		/** Indicate if the button is enabled or not (grayed) */
		bool bEnabled = true;

		/** A Button Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FButtonStyle* Style = nullptr;
	};

	// Sync with FSlateIMCheckBoxParams
	struct FCheckBoxParams
	{
		/** The label for the checkbox */
		const FStringView Label;

		/** A Checkbox Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FCheckBoxStyle* CheckBoxStyle = nullptr;
	};

	template<typename InNumericType>
	struct FSpinBoxParams
	{
		/** The minimum value of the spin box (or unset for no limit). */
		TOptional<InNumericType> Min;

		/** The maximum value of the spin box (or unset for no limit). */
		TOptional<InNumericType> Max;

		/** A Spin Box Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FSpinBoxStyle* Style = nullptr;
	};

	// Sync with FSlateIMSliderParams
	struct FSliderParams
	{
		/** The minimum value of the slider */
		float Min = 0.f;

		/** The maximum value of the slider */
		float Max = 100.f;

		/** The smallest incremental change that can be made to the slider's value */
		float Step = 1.f;

		/** A Slider Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FSliderStyle* Style = nullptr;
	};

	// Sync with FSlateIMProgressBarParams
	struct FProgressBarParams
	{
		/** A Progress Bar Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FProgressBarStyle* Style = nullptr;
	};

	// Sync with FSlateIMComboBoxParams
	struct FComboBoxParams
	{
		/** Whether to force a refresh of the available options or the selected option (set to true for a frame when changing
		 * the list of options or manually setting the selected index) */
		bool bForceRefresh = false;

		/** Adds a search bar to the combo box */
		bool bSearchable = false;

		/** A Combo Box Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FComboBoxStyle* Style = nullptr;
	};

	// Sync with FSlateIMSelectionListParams
	struct FSelectionListParams
	{
		/** Whether to force a refresh of the available options or the selected option (set to true for a frame when changing
		 * the list of options or manually setting the selected index) */
		bool bForceRefresh = false;

		/** A Table View Style to override the default style. Changes to the style after the first invocation are not reflected,
		 * provide a new style object to update the visuals at runtime */
		const FTableViewStyle* Style = nullptr;
	};

	// Sync with FSlateIMMenuButtonParams
	struct FMenuButtonParams
	{
		/** The tooltip to display for the menu item */
		FStringView ToolTipText = FStringView();
	};

	// Sync with FSlateIMTabParams
	struct FTabParams
	{
		/** An icon to display on the tab next to the title. This cannot be updated after the initial creation of the tab. */
		const FSlateIcon TabIcon = FSlateIcon();

		/** The title to display for the tab if it should differ from the TabId. This cannot be updated after the initial creation of the tab. */
		const FText TabTitle;
	};

	// Sync with FSlateIMModalDialogParams
	struct FModalDialogParams
	{
		/** The type of dialog to display */
		EAppMsgCategory Category = EAppMsgCategory::Warning;

		/** The title to display in the dialog window */
		const FStringView Title = FStringView();
	};
} // SlateIM

USTRUCT(BlueprintType)
struct FSlateIMViewportRootLayout
{
	GENERATED_BODY()

	/** How to anchor the root within the viewport (normalized units) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FAnchors Anchors = FAnchors(0);

	/** Offset the root from the anchored position in the viewport (in slate units) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FVector2f Offset = FVector2f::ZeroVector;

	/** How the root is aligned to its anchor (normalized units) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FVector2f Alignment = FVector2f::ZeroVector;

	/** Optional set the size (in slate units) of the root in the viewport. When unset, the root will auto-size for its content. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<FVector2f> Size;

	/** The ZOrder of the SlateIM widget in the viewport */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	int32 ZOrder = 10000;

	/** Global scale to apply within the viewport */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	float Scale = 1.f;
};

USTRUCT(BlueprintType)
struct FSlateIMWindowParams
{
	GENERATED_BODY()

	// The name of the window. This will also be displayed in the windows title bar
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FString WindowTitle;

	// The size to open the window at (does not update the size of an existing window)
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FVector2f WindowSize = FVector2f::ZeroVector;

	// If this window was created with a previous call to BeginWindow and then closed, passing in true will reopen the window and false will leave it closed. If the window has never been seen this parameter does nothing and the window will open
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bShouldReopen = false;

	// Window will be create always on top.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bAlwaysOnTop = false;

	operator SlateIM::FWindowParams() const
	{
		return {
			WindowTitle,
			WindowSize,
			bShouldReopen,
			bAlwaysOnTop
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMViewportParams
{
	GENERATED_BODY()

	/** How to lay out the root within the viewport */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FSlateIMViewportRootLayout Layout;
};

USTRUCT(BlueprintType)
struct FSlateIMTableParams
{
	GENERATED_BODY()

	/** Select mode for the table. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TEnumAsByte<ESelectionMode::Type> SelectionMode = ESelectionMode::None;

	operator SlateIM::FTableParams() const
	{
		return {
			nullptr,
			nullptr,
			SelectionMode.GetValue()
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMTableColumnParams
{
	GENERATED_BODY()

	/** Added a default label to the column. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FString Label;

	operator SlateIM::FTableColumnParams() const
	{
		return { Label };
	}
};

USTRUCT(BlueprintType)
struct FSlateIMTableRowChildrenParams
{
	GENERATED_BODY()

	/** Id of the parent row. 0 to disable state saving. Row ids must be unique to its parent. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	int32 ParentRowId = 0;

	/** Set the default expansion state of the parent (whether the children following are visible.) Only applies to rows with a parent id. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bDefaultExpanded = false;

	operator SlateIM::FTableRowChildrenParams() const
	{
		return {
			static_cast<uint32>(ParentRowId),
			bDefaultExpanded
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMBorderParams
{
	GENERATED_BODY()

	/** Which direction the contents of the container should flow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Vertical;

	/** Whether the container should handle all mouse inputs */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bAbsorbMouse = true;

	/** How much to pad the contents of the container */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FMargin ContentPadding = FMargin(2.0f);
};

USTRUCT(BlueprintType)
struct FSlateIMScrollBoxParams
{
	GENERATED_BODY()

	/** Which direction content should flow and scroll */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Vertical;
};

USTRUCT(BlueprintType)
struct FSlateIMPopUpParams
{
	GENERATED_BODY()

	/** Which direction the contents of the container should flow */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Vertical;

	/** Whether the container should handle all mouse inputs */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bAbsorbMouse = true;

	/** How much to pad the contents of the container */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FMargin ContentPadding = FMargin(2.0f);
};

USTRUCT(BlueprintType)
struct FSlateIMTextParams
{
	GENERATED_BODY()

	/** The color to display the text as */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FSlateColor Color = FSlateColor::UseForeground();

	operator SlateIM::FTextParams() const
	{
		return {
			Color,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMEditableTextParams
{
	GENERATED_BODY()

	/** HintText The hint text to display when the input field is empty */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FString HintText;

	operator SlateIM::FEditableTextParams() const
	{
		return {
			HintText,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMImageParams
{
	GENERATED_BODY()

	/** The color to tint the brush */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FSlateColor ColorAndOpacity = FLinearColor::White;

	/** Override the desired size to display the brush at */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FVector2D DesiredSize = FVector2D::ZeroVector;
};

USTRUCT(BlueprintType)
struct FSlateIMButtonParams
{
	GENERATED_BODY()

	/** Indicate if the button is enabled or not (grayed) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bEnabled = true;

	operator SlateIM::FButtonParams() const
	{
		return { 
			bEnabled,
			nullptr 
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMCheckBoxParams
{
	GENERATED_BODY()

	/** The label for the checkbox */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FString Label;

	operator SlateIM::FCheckBoxParams() const
	{
		return { 
			Label,
			nullptr 
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMSpinBoxFloatParams
{
	GENERATED_BODY()

	/** The minimum value of the spin box(or unset for no limit). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<float> Min;

	/** The maximum value of the spin box(or unset for no limit). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<float> Max;

	operator SlateIM::FSpinBoxParams<float>() const
	{
		return {
			Min,
			Max,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMSpinBoxDoubleParams
{
	GENERATED_BODY()

	/** The minimum value of the spin box(or unset for no limit). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<double> Min;

	/** The maximum value of the spin box(or unset for no limit). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<double> Max;

	operator SlateIM::FSpinBoxParams<double>() const
	{
		return {
			Min,
			Max,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMSpinBoxInt32Params
{
	GENERATED_BODY()

	/** The minimum value of the spin box(or unset for no limit). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<int32> Min;

	/** The maximum value of the spin box(or unset for no limit). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	TOptional<int32> Max;

	operator SlateIM::FSpinBoxParams<int32>() const
	{
		return {
			Min,
			Max,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMSliderParams
{
	GENERATED_BODY()

	/** The minimum value of the slider */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	float Min = 0.f;

	/** The maximum value of the slider */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	float Max = 100.f;

	/** The smallest incremental change that can be made to the slider's value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	float Step = 1.f;

	operator SlateIM::FSliderParams() const
	{
		return {
			Min,
			Max,
			Step,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMProgressBarParams
{
	GENERATED_BODY()

	operator SlateIM::FProgressBarParams() const
	{
		return { nullptr };
	}
};

USTRUCT(BlueprintType)
struct FSlateIMComboBoxParams
{
	GENERATED_BODY()

	/** Whether to force a refresh of the available options or the selected option (set to true for a frame when changing
	 * the list of options or manually setting the selected index) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bForceRefresh = false;

	/** Adds a search bar to the combo box */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bSearchable = false;

	operator SlateIM::FComboBoxParams() const
	{
		return {
			bForceRefresh,
			bSearchable,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMSelectionListParams
{
	GENERATED_BODY()

	/** Whether to force a refresh of the available options or the selected option (set to true for a frame when changing
	 * the list of options or manually setting the selected index) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	bool bForceRefresh = false;

	operator SlateIM::FSelectionListParams() const
	{
		return {
			bForceRefresh,
			nullptr
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMMenuButtonParams
{
	GENERATED_BODY()

	/** The tooltip to display for the menu item */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FString ToolTipText;

	operator SlateIM::FMenuButtonParams() const
	{
		return { ToolTipText };
	}
};

USTRUCT(BlueprintType)
struct FSlateIMIcon
{
	GENERATED_BODY()

	/** The name of the style set the icon can be found in. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FName StyleSetName = NAME_None;

	/** The name of the style for the icon. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FName StyleName = NAME_None;

	/** The name of the style for the small icon. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FName SmallStyleName = NAME_None;

	/** Name of the style for the status overlay icon */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FName StatusOverlayStyleName = NAME_None;
};

USTRUCT(BlueprintType)
struct FSlateIMTabParams
{
	GENERATED_BODY()

	/** An icon to display on the tab next to the title. This cannot be updated after the initial creation of the tab. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FSlateIMIcon TabIcon;

	/** The title to display for the tab if it should differ from the TabId. This cannot be updated after the initial creation of the tab. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FText TabTitle;

	operator SlateIM::FTabParams() const
	{
		return {
			FSlateIcon(TabIcon.StyleSetName, TabIcon.StyleName, TabIcon.SmallStyleName, TabIcon.StatusOverlayStyleName),
			TabTitle
		};
	}
};

USTRUCT(BlueprintType)
struct FSlateIMModalDialogParams
{
	GENERATED_BODY()

	/** The type of dialog to display */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	EAppMsgCategory Category = EAppMsgCategory::Warning;

	/** The title to display in the dialog window */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FString Title;

	operator SlateIM::FModalDialogParams() const
	{
		return {
			Category,
			Title
		};
	}
};

UENUM(BlueprintType)
enum class ESlateIMFocusDepth : uint8
{
	// Only check if the previous widget is focused
	SelfOnly,

	// Check if the previous widget or any of its childs widgets are focused (recursively)
	IncludingDescendants
};

USTRUCT(BlueprintType)
struct FSlateIMGraphLinePointsParams
{
	GENERATED_BODY()

	/** The min and max X values to horizontally scale the graph to */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FDoubleRange XViewRange = FDoubleRange(0.0, 1.0);

	/** The min and max Y values to vertically scale the graph to */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FDoubleRange YViewRange = FDoubleRange(0.0, 1.0);

	/** The color of the line to draw for this graph */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FLinearColor LineColor = FLinearColor::White;

	/** How thick to draw the line */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	float LineThickness = 1.f;
};

USTRUCT(BlueprintType)
struct FSlateIMGraphLineValuesParams
{
	GENERATED_BODY()

	/** The min and max Y values to vertically scale the graph to */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FDoubleRange ViewRange = FDoubleRange(0.0, 1.0);

	/** The color of the line to draw for this graph */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	FLinearColor LineColor = FLinearColor::White;

	/** How thick to draw the line */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIM")
	float LineThickness = 1.f;
};
