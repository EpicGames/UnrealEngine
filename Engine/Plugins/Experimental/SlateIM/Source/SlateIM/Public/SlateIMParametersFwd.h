// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FButtonStyle;
struct FCheckBoxStyle;
struct FComboBoxStyle;
struct FEditableTextBoxStyle;
struct FProgressBarStyle;
struct FSliderStyle;
struct FSpinBoxStyle;
struct FTableRowStyle;
struct FTableViewStyle;
struct FTextBoxStyle;

struct FSlateIMViewportRootLayout;
struct FSlateIMWindowParams;
struct FSlateIMViewportParams;
struct FSlateIMTableParams;
struct FSlateIMTableColumnParams;
struct FSlateIMTableRowChildrenParams;
struct FSlateIMBorderParams;
struct FSlateIMScrollBoxParams;
struct FSlateIMPopUpParams;
struct FSlateIMTextParams;
struct FSlateIMEditableTextParams;
struct FSlateIMImageParams;
struct FSlateIMButtonParams;
struct FSlateIMCheckBoxParams;
struct FSlateIMSpinBoxFloatParams;
struct FSlateIMSpinBoxDoubleParams;
struct FSlateIMSpinBoxInt32Params;
struct FSlateIMSliderParams;
struct FSlateIMProgressBarParams;
struct FSlateIMComboBoxParams;
struct FSlateIMSelectionListParams;
struct FSlateIMMenuButtonParams;
struct FSlateIMIcon;
struct FSlateIMTabParams;
struct FSlateIMModalDialogParams;
enum class ESlateIMFocusDepth : uint8;
struct FSlateIMGraphLinePointsParams;
struct FSlateIMGraphLineValuesParams;

#if WITH_ENGINE
enum class ESlateIMEngineCanvasUpdateType : uint8;
struct FSlateIMEngineCanvasParams;
struct FSlateIMEngineTileRenderParams;
struct FSlateIMEngineTextRenderParams;
struct FSlateIMEngineBorderRenderParams;
#endif

namespace SlateIM
{
	using FViewportRootLayout = FSlateIMViewportRootLayout;
	using FViewportParams = FSlateIMViewportParams;
	using FBorderParams = FSlateIMBorderParams;
	using FScrollBoxParams = FSlateIMScrollBoxParams;
	using FPopUpParams = FSlateIMPopUpParams;
	using FImageParams = FSlateIMImageParams;
	using EFocusDepth = ESlateIMFocusDepth;
	using FGraphLinePointsParams = FSlateIMGraphLinePointsParams;
	using FGraphLineValuesParams = FSlateIMGraphLineValuesParams;

#if WITH_ENGINE
	namespace Canvas
	{
		using ECanvasUpdateType = ESlateIMEngineCanvasUpdateType;
		using FCanvasParams = FSlateIMEngineCanvasParams;
		using FTileRenderParams = FSlateIMEngineTileRenderParams;
		using FTextRenderParams = FSlateIMEngineTextRenderParams;
		using FBorderRenderParams = FSlateIMEngineBorderRenderParams;
	}
#endif

	struct FWindowParams;
	struct FTableParams;
	struct FTableColumnParams;
	struct FTableRowChildrenParams;
	struct FTextParams;
	struct FEditableTextParams;
	struct FButtonParams;
	struct FCheckBoxParams;
	template<typename InNumericType> struct FSpinBoxParams;
	struct FSliderParams;
	struct FProgressBarParams;
	struct FComboBoxParams;
	struct FSelectionListParams;
	struct FMenuButtonParams;
	struct FTabParams;
	struct FModalDialogParams;
} // SlateIM
