// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "HAL/IConsoleManager.h"
#include "Math/Vector2D.h"
#include "Textures/SlateIcon.h"

#define UE_API SLATEIM_API

class SDockTab;
class SWidget;

/**
 * A utility base class for creating SlateIM-based tools.
 * Handles setting up a tick for drawing the tool.
 * 
 * Note: this class is not required for using SlateIM but provides common functionality for SlateIM tools
 *
 * The test widgets (FSlateIMTestViewportWidget/FSlateIMTestWindowWidget in SlateIMManager) are examples of using this class.
 */
class FSlateIMWidgetBase
{
public:
	UE_API explicit FSlateIMWidgetBase(const FStringView& Name);
	UE_API virtual ~FSlateIMWidgetBase();

	void ToggleWidget() { IsWidgetEnabled() ? DisableWidget() : EnableWidget(); }
	UE_API virtual void EnableWidget();
	UE_API virtual void DisableWidget();

	bool IsWidgetEnabled() const { return TickHandle.IsValid(); }

	const FName& GetWidgetName() const { return WidgetName; }

protected:
	virtual void DrawWidget(float DeltaTime) = 0;

private:
	FDelegateHandle TickHandle;
	FName WidgetName;
};

/**
 * Specialized version of FSlateIMWidgetBase that auto registers a command to toggle this widget
 */
class FSlateIMWidgetWithCommandBase : public FSlateIMWidgetBase
{
public:
	UE_API FSlateIMWidgetWithCommandBase(const TCHAR* Command, const TCHAR* CommandHelp);
	UE_API FSlateIMWidgetWithCommandBase(const FStringView& WidgetName, const TCHAR* Command, const TCHAR* CommandHelp);

private:
	FAutoConsoleCommand WidgetCommand;
};

/**
 * Specialized version of FSlateIMWidgetBase that draws in a window and disables itself when the user closes the window
 */
class FSlateIMWindowBase : public FSlateIMWidgetWithCommandBase
{
public:
	FSlateIMWindowBase(FStringView WindowTitle, FVector2f WindowSize, const TCHAR* Command, const TCHAR* CommandHelp)
		: FSlateIMWindowBase(WindowTitle, WindowSize, /* Always On Top */ false, Command, CommandHelp)
	{}

	FSlateIMWindowBase(FStringView WindowTitle, FVector2f WindowSize, bool bAlwaysOnTop, const TCHAR* Command, const TCHAR* CommandHelp)
		: FSlateIMWidgetWithCommandBase(Command, CommandHelp)
		, WindowTitle(WindowTitle)
		, WindowSize(WindowSize)
		, bAlwaysOnTop(bAlwaysOnTop)
	{}

protected:
	virtual void DrawWindow(float DeltaTime) = 0;

private:
	UE_API virtual void DrawWidget(float DeltaTime) final override;

	FString WindowTitle;
	FVector2f WindowSize;
	bool bAlwaysOnTop;
};

/**
 * Specialized version of FSlateIMWidgetBase that draws in a nomad (dockable) tab and disables itself when the user closes the window
 */
class FSlateIMNomadTabBase : public FSlateIMWidgetWithCommandBase
{
public:
	UE_API FSlateIMNomadTabBase(FStringView TabTitle, const TCHAR* Command, const TCHAR* CommandHelp, FSlateIcon TabIcon = FSlateIcon());
	UE_API virtual ~FSlateIMNomadTabBase();
	
	UE_API virtual void DisableWidget() override;

protected:
	virtual void DrawContent(float DeltaTime) = 0;

private:
	UE_API virtual void DrawWidget(float DeltaTime) final override;

	TWeakPtr<SDockTab> RootTab;
	TWeakPtr<SWidget> TabContent;
};

/**
 * Specialized version of FSlateIMWidgetBase that exposes the resulting widget for embedding in other slate widgets
 *
 * @note EnableWidget() must be called to get a valid ExposedWidget
 */
class FSlateIMExposedBase : public FSlateIMWidgetBase
{
public:
	explicit FSlateIMExposedBase(const FStringView& Name)
		: FSlateIMWidgetBase(Name)
	{
	}

	// Get the widget to embed in your existing slate hierarchy
	UE_API TSharedRef<SWidget> GetExposedWidget() const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExposedWidgetChanged, TSharedRef<SWidget>);
	FOnExposedWidgetChanged OnExposedWidgetChanged;

protected:
	virtual void DrawContent(float DeltaTime) = 0;

private:
	UE_API virtual void DrawWidget(float DeltaTime) final override;

	TSharedPtr<SWidget> ExposedWidget;
	
};

#undef UE_API
