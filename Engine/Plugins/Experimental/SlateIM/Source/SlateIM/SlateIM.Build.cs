// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateIM : ModuleRules
{
	public SlateIM(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Slate",
				"SlateCore",
				"InputCore",
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("SlateIMEngine");

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
				});
		}

		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
				});
		}
		
		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PublicDefinitions.Add("WITH_SLATEIM_EXAMPLES=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_SLATEIM_EXAMPLES=0");
		}
	}
}
