// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Containers/StringFwd.h"
#include "SlateIM.h"
#include "SlateIMParametersFwd.h"

#include "SlateIMBlueprintFunctionLibrary.generated.h"

class UGameViewportClient;
class ULocalPlayer;
class UMaterialInterface;
class UTexture;
class UTexture2D;
class UTextureRenderTarget2D;

UCLASS(BlueprintType, ClassGroup = "SlateIM", DisplayName = "SlateIM")
class USlateIMBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Begins a new floating window root
	 *
	 * @param UniqueName a unique name to identify this window
	 * @param Params Configuration parameters for the window.
	 *
	 * @return the current window state. True if open, false if closed or not updating
	 *
	 * @note Consider using FSlateIMWindowBase to handle creating and updating your Slate IM Window
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Roots")
	static bool BeginWindowRoot(FName UniqueName, const FSlateIMWindowParams& Params)
	{
		return SlateIM::BeginWindowRoot(UniqueName, Params);
	}

	/**
	 * Begins a new root in the game viewport
	 *
	 * @param UniqueName a globally unique name to identify this root
	 * @param ViewportClient the viewport client to add the widget to
	 * @param Params Parameters used to create the root.
	 *
	 * @return Whether the root is valid and updating
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Roots")
	static bool BeginViewportClientRoot(FName UniqueName, UGameViewportClient* ViewportClient, const FSlateIMViewportParams& Params)
	{
		return SlateIM::BeginViewportRoot(UniqueName, ViewportClient, Params);
	}

	/**
	 * Begins a new root in the provided player's viewport
	 *
	 * Please use the method below with the struct-parameter.
	 * 
	 * @param UniqueName a globally unique name to identify this root
	 * @param LocalPlayer the player whose viewport to add the widget to
	 * @param Layout How to lay out the root within the viewport
	 * 
	 * @return Whether the root is valid and updating
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Roots")
	static bool BeginViewportPlayerRoot(FName UniqueName, ULocalPlayer* LocalPlayer, const FSlateIMViewportParams& Params)
	{
		return SlateIM::BeginViewportRoot(UniqueName, LocalPlayer, Params);
	}

	/**
	 * Ends any Root type, must always be called regardless of the result of the Begin function
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Roots")
	static void EndRoot()
	{
		SlateIM::EndRoot();
	}

	/**
	 * Begins a horizontally stacked container. All widgets created within the Horizontal Stack container will be placed in-order left-to-right
	 *
	 * @param bMaximizeContent Maximize() the contents of the stack
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Stack")
	static void BeginHorizontalStack(bool bMaximizeContent = false)
	{
		SlateIM::BeginHorizontalStack(bMaximizeContent);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Stack")
	static void EndHorizontalStack()
	{
		SlateIM::EndHorizontalStack();
	}

	/**
	 * Begins a vertically stacked container. All widgets created within the Vertical Stack container will be placed in-order top-to-bottom
	 *
	 * @param bMaximizeContent Maximize() the contents of the stack
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Stack")
	static void BeginVerticalStack(bool bMaximizeContent = false)
	{
		SlateIM::BeginVerticalStack(bMaximizeContent);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Stack")
	static void EndVerticalStack()
	{
		SlateIM::EndVerticalStack();
	}

	/**
	 * Begins a horizontally wrapped container. All widgets created within the Horizontal Wrap container will be placed in-order left-to-right until the allotted width is filled,
	 * then the content will begin a new line and so-on.
	 *
	 * @param bMaximizeContent Maximize() the contents of the wrap
	 *
	 * @note Wrap box children are always AutoSize and do not support Fill
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Wrap")
	static void BeginHorizontalWrap(bool bMaximizeContent = false)
	{
		SlateIM::BeginHorizontalWrap(bMaximizeContent);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Wrap")
	static void EndHorizontalWrap()
	{
		SlateIM::EndHorizontalWrap();
	}

	/**
	 * Begins a vertically wrapped container. All widgets created within the Vertical Wrap container will be placed in-order top-to-bottom until the allotted height is filled,
	 * then the content will begin a new column and so-on.
	 *
	 * @param bMaximizeContent Maximize() the contents of the wrap
	 *
	 * @note Wrap container children are always AutoSize and do not support Fill
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Wrap")
	static void BeginVerticalWrap(bool bMaximizeContent = false)
	{
		SlateIM::BeginVerticalWrap(bMaximizeContent);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Wrap")
	static void EndVerticalWrap()
	{
		SlateIM::EndVerticalWrap();
	}

	/**
	 * Begin a Table container. The contents are automatically scrollable if the table is sized smaller than all its content.
	 *
	 * @param Params Settings for table construction.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void BeginTable(const FSlateIMTableParams& Params)
	{
		SlateIM::BeginTable(Params);
	}

	/**
	 * End the current Table container.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void EndTable()
	{
		SlateIM::EndTable();
	}

	/**
	 * Starts the header. Call this before adding any content to the table.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void BeginTableHeader()
	{
		SlateIM::BeginTableHeader();
	}

	/**
	 * Ends the table header.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void EndTableHeader()
	{
		SlateIM::EndTableHeader();
	}

	/**
	 * Moves to the next column. Allows fine control of the column body.
	 *
	 * @param ColumnID The name to use as ID for the column
	 *
	 * @note Table columns support SetToolTip, HAlign, and VAlign. See FixedTableColumnWidth() and InitialTableColumnWidth() for size options.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void NextTableColumn(const FName& ColumnID)
	{
		SlateIM::NextTableColumn(ColumnID);
	}

	/**
	 * Adds a column and immediately ends it with the given label as its body.
	 *
	 * @param ColumnID The name to use as ID for the column
	 * @param Params The params used to create the column
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void AddTableColumn(const FName& ColumnID, const FSlateIMTableColumnParams& Params)
	{
		SlateIM::AddTableColumn(ColumnID, Params);
	}

	/**
	 * Set the column to a fixed width, not resizable by users. Call this before calling AddTableColumn().
	 *
	 * @param Width The fixed width of the column
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void FixedTableColumnWidth(float Width)
	{
		SlateIM::FixedTableColumnWidth(Width);
	}

	/**
	 * Set the column to an initial width, which can then be resized by users. Call this before calling AddTableColumn().
	 *
	 * @param Width The initial width of the column
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void InitialTableColumnWidth(float Width)
	{
		SlateIM::InitialTableColumnWidth(Width);
	}

	/**
	 * Starts a table body.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void BeginTableBody()
	{
		SlateIM::BeginTableBody();
	}

	/**
	 * Ends a table body.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void EndTableBody()
	{
		SlateIM::EndTableBody();
	}

	/**
	 * Begin the content for the next table cell. The column and row are tracked internally based on the number of columns.
	 *
	 * @param bOutRowSelected If supplied, will return true if the row this cell belongs to is selected.
	 *
	 * @return Whether the contents of this table cell are visible. When false, the content can be skipped to save cycles.
	 *
	 * @note There is no "End" function counterpart to this call. This function supports the various slot functions (Padding, HAlign, VAlign, Min/Max Width/Height, etc).
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static bool NextTableCell(bool& bOutRowSelected)
	{
		return SlateIM::NextTableCell(&bOutRowSelected);
	}

	/**
	 * Begin adding content as a child to the current table row. NextTableCell() must still be called after this function.
	 * If the table is a tree, a unique row id must be provided when starting a new set of child rows so that the expansion
	 * state of the parent row is saved. Not providing a row id will cause the parent row to revert to the default expansion
	 * state if its grandparent row is closed.
	 *
	 * @param Params Params used to begin the table children.
	 *
	 * @return Whether the parent row is expanded or not. When false, the child content can be skipped to save cycles.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static bool BeginTableRowChildren(const FSlateIMTableRowChildrenParams& Params)
	{
		return SlateIM::BeginTableRowChildren(Params);
	}

	/**
	 * Stop adding child content to the parent row. This must be called no matter the result of BeginTableRowChildren()
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Table")
	static void EndTableRowChildren()
	{
		SlateIM::EndTableRowChildren();
	}

	/**
	 * Begin a container with a background image.
	 *
	 * @param BorderStyleName The name of the brush in FAppStyle to use as the background for this container
	 * @param Params Additional params to construct this border
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Border")
	static void BeginBorder(FName BorderStyleName, const FSlateIMBorderParams& Params)
	{
		SlateIM::BeginBorder(BorderStyleName, Params);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Border")
	static void EndBorder()
	{
		SlateIM::EndBorder();
	}

	/**
	 * Begins a container that will allow the user to scroll when its content is larger than the allotted space.
	 *
	 * @param Params Parameters to create the scroll box
	 *
	 * @return Whether the user scroll the content this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Scroll")
	static bool BeginScrollBox(const FSlateIMScrollBoxParams& Params)
	{
		return SlateIM::BeginScrollBox(Params);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Scroll")
	static void EndScrollBox()
	{
		SlateIM::EndScrollBox();
	}

	/**
	 * Begin a container with a background image.
	 *
	 * @param BorderStyleName The name of the brush in FAppStyle to use as the background for this container
	 * @param Params Additional params to construct this border
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|PopUp", meta=(BorderStyleName= "ToolPanel.GroupBorder"))
	static void BeginPopUp(const FName BorderStyleName, const FSlateIMPopUpParams& Params)
	{
		SlateIM::BeginPopUp(BorderStyleName, Params);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|PopUp")
	static void EndPopUp()
	{
		SlateIM::EndPopUp();
	}

	/**
	 * Sets the padding for the next widget
	 *
	 * @param NextPadding The padding to apply to the next widget
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void Padding(const FMargin NextPadding)
	{
		SlateIM::Padding(NextPadding);
	}

	/**
	 * Sets the horizontal alignment for the next widget
	 *
	 * @param NextAlignment The alignment to apply to the next widget
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void HAlign(EHorizontalAlignment NextAlignment)
	{
		SlateIM::HAlign(NextAlignment);
	}

	/**
	 * Sets the vertical alignment for the next widget
	 *
	 * @param NextAlignment The alignment to apply to the next widget
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void VAlign(EVerticalAlignment NextAlignment)
	{
		SlateIM::VAlign(NextAlignment);
	}

	/**
	 * Set the next slot to AutoSize to its content
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void AutoSize()
	{
		SlateIM::AutoSize();
	}

	/**
	 * Sets the next slot to Fill the remaining space in its container
	 *
	 * @note The filled space is shared equally with any other slots set to Fill the same container
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void Fill()
	{
		SlateIM::Fill();
	}

	/**
	 * Set the minimum width the next slot can have.
	 *
	 * @param InMinWidth The minimum width for the next slot
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void MinWidth(float MinWidth)
	{
		SlateIM::MinWidth(MinWidth);
	}

	/**
	 * Set the minimum height the next slot can have.
	 *
	 * @param InMinHeight The minimum height for the next slot
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void MinHeight(float MinHeight)
	{
		SlateIM::MinHeight(MinHeight);
	}

	/**
	 * Set the maximum width the next slot can have.
	 *
	 * @param InMaxWidth The maximum width for the next slot
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void MaxWidth(float MaxWidth)
	{
		SlateIM::MaxWidth(MaxWidth);
	}

	/**
	 * Set the maximum height the next slot can have.
	 *
	 * @param InMaxHeight The maximum height for the next slot
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void MaxHeight(float MaxHeight)
	{
		SlateIM::MaxHeight(MaxHeight);
	}

	/**
	 * Set the alignments to Fill and removes autosize.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers|Slots")
	static void Maximize()
	{
		SlateIM::Maximize();
	}

	/**
	 * Display a string of text
	 *
	 * @param InText The text string to display
	 * @param Params Additional parameters to create the text.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Text")
	static void Text(const FString& InText, const FSlateIMTextParams& Params)
	{
		SlateIM::Text(InText, Params);
	}

	/**
	 * Create a text input field
	 *
	 * @param InOutText The text that is in the input field. Can have a default value
	 * @param Params Additional parameters to create this widget
	 *
	 * @return Whether the user changed or committed text this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Text")
	static bool EditableText(UPARAM(Ref) FString& InOutText, const FSlateIMEditableTextParams& Params)
	{
		return SlateIM::EditableText(InOutText, Params);
	}

	/**
	 * Display a slate style brush
	 *
	 * @param ImageStyleName The named of the brush to display from FAppStyle
	 * @param Params Additional parameters to display this image
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Image")
	static void ImageName(const FName ImageStyleName, const FSlateIMImageParams& Params)
	{
		SlateIM::Image(ImageStyleName, Params);
	}

	/**
	 * Display a colored box
	 *
	 * @param Params Additional parameters to display this image (including the color)
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Image")
	static void ImageBlock(const FSlateIMImageParams& Params)
	{
		SlateIM::Image(Params);
	}

	/**
	 * Display a texture
	 *
	 * @param ImageTexture The texture to display
	 * @param Params Additional parameters to display this image
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Image")
	static void Image(UTexture2D* ImageTexture, const FSlateIMImageParams& Params)
	{
		SlateIM::Image(ImageTexture, Params);
	}

	/**
	 * Display a render target texture
	 *
	 * @param ImageRenderTarget The render target texture to display
	 * @param Params Additional parameters to display this image
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Image")
	static void ImageTexture(UTextureRenderTarget2D* ImageRenderTarget, const FSlateIMImageParams& Params)
	{
		SlateIM::Image(ImageRenderTarget, Params);
	}

	/**
	 * Display a material
	 *
	 * @param ImageMaterial The material to display
	 * @param BrushSize The size to create the internal brush with
	 * @param Params Additional parameters to display this image
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Image")
	static void ImageMaterial(UMaterialInterface* ImageMaterial, FVector2D BrushSize, const FSlateIMImageParams& Params)
	{
		SlateIM::Image(ImageMaterial, BrushSize, Params);
	}

	/**
	 * Display a button with text
	 *
	 * @param InText The text to display on the button
	 * @param Params Additional parameters to construct this button
	 *
	 * @return Whether the user clicked the button this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool Button(const FString& InText, const FSlateIMButtonParams& Params)
	{
		return SlateIM::Button(InText, Params);
	}

	/**
	 * Display a two-state checkbox
	 *
	 * @param InOutCurrentState The current state of the checkbox. Can provide a default value.
	 * @param Params Additional params to create the checkbox
	 *
	 * @return Whether the user changed the state of the checkbox this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool CheckBox(UPARAM(Ref) bool& InOutCurrentState, const FSlateIMCheckBoxParams& Params)
	{
		return SlateIM::CheckBox(InOutCurrentState, Params);
	}

	/**
	 * Display a float-based spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params Additional parameters to create this spinbox
	 * 
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool SpinBoxFloat(UPARAM(Ref) float& InOutValue, const FSlateIMSpinBoxFloatParams& Params)
	{
		return SlateIM::SpinBox<float>(InOutValue, Params);
	}

	/**
	 * Display a double-based spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params Additional parameters to create this spinbox
	 *
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool SpinBoxDouble(UPARAM(Ref) double& InOutValue, const FSlateIMSpinBoxDoubleParams& Params)
	{
		return SlateIM::SpinBox<double>(InOutValue, Params);
	}

	/**
	 * Display an int32-based spin box
	 *
	 * @param InOutValue The current value of the spin box. Can provide a default value.
	 * @param Params Additional parameters to create this spinbox
	 *
	 * @return Whether the user changed or committed the spin box value this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool SpinBoxInt32(UPARAM(Ref) int32& InOutValue, const FSlateIMSpinBoxInt32Params& Params)
	{
		return SlateIM::SpinBox<int32>(InOutValue, Params);
	}	
	
	/**
	 * Display a float-based slider
	 *
	 * @param InOutValue The current value of the slider. Can provide a default value.
	 * @param Params Additional parameters to construct this slider
	 *
	 * @return Whether the user changed the value of the slider this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool Slider(UPARAM(Ref) float& InOutValue, const FSlateIMSliderParams& Params)
	{
		return SlateIM::Slider(InOutValue, Params);
	}

	/**
	 * Display a progress bar
	 *
	 * @param Percent The value of the progress bar (0.0 to 1.0)
	 * @param Params Additional params to construct the progress bar
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static void ProgressBar(TOptional<float> Percent, const FSlateIMProgressBarParams& Params)
	{
		SlateIM::ProgressBar(Percent, Params);
	}

	/**
	 * Display a dropdown of text options
	 *
	 * @param ComboItems The options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param Params Additional parameter to construct this combo box
	 *
	 * @return Whether the user changed the selected option this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool ComboBox(const TArray<FString>& ComboItems, UPARAM(Ref) int32& InOutSelectedItemIndex, const FSlateIMComboBoxParams& Params)
	{
		return SlateIM::ComboBox(ComboItems, InOutSelectedItemIndex, Params);
	}
	
	/**
	 * Display a list of text options
	 *
	 * @param ListItems The list of options the user can choose from
	 * @param InOutSelectedItemIndex The index of the currently selected option
	 * @param Params Additional parameters used to construct this selection list
	 *
	 * @return Whether the user changed the selected option this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Controls")
	static bool SelectionList(const TArray<FString>& ListItems, UPARAM(Ref) int32& InOutSelectedItemIndex, const FSlateIMSelectionListParams& Params)
	{
		return SlateIM::SelectionList(ListItems, InOutSelectedItemIndex, Params);
	}

	/**
	 * Create a block of empty space
	 *
	 * @param Size The size of whitespace to create
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Misc")
	static void Spacer(const FVector2D& Size)
	{
		SlateIM::Spacer(Size);
	}

	/**
	 * Begin an area where a menu appears with a right click
	 *
	 * @return Whether the menu is open
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static bool BeginContextMenuAnchor()
	{
		return SlateIM::BeginContextMenuAnchor();
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void EndContextMenuAnchor()
	{
		SlateIM::EndContextMenuAnchor();
	}

	/**
	* Begin a menu bar, similar to those shown at the top of windows. Can be placed anywhere.
	*
	* @see AddMenuSeparator AddMenuButton AddMenuToggleButton AddMenuCheckButton BeginSubMenu
	*/
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void BeginMenuBar()
	{
		SlateIM::BeginMenuBar();
	}

	/**
	 * End the menu bar.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void EndMenuBar()
	{
		SlateIM::EndMenuBar();
	}

	/**
	 * Adds a menu to the menu entry with the given name.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void AddMenuBarEntry(const FString& InMenuName)
	{
		SlateIM::AddMenuBarEntry(InMenuName);
	}

	/**
	 * Ends a menu bar entry
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void EndMenuBarEntry()
	{
		SlateIM::EndMenuBarEntry();
	}

	/**
	 * Add a new section to the current context menu
	 *
	 * @param SectionText The label of the section to add
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void AddMenuSection(const FString& SectionText)
	{
		SlateIM::AddMenuSection(SectionText);
	}

	/**
	 * Add a separator to the current context menu
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void AddMenuSeparator()
	{
		SlateIM::AddMenuSeparator();
	}

	/**
	 * Add a menu item button to the current context menu
	 *
	 * @param RowText Label for the menu item
	 * @param Params Additional parameters used to construct the menu button
	 *
	 * @return Whether the user clicked the menu item this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static bool AddMenuButton(const FString& RowText, const FSlateIMMenuButtonParams& Params)
	{
		return SlateIM::AddMenuButton(RowText, Params);
	}

	/**
	 * Display a menu item button with a checkbox
	 *
	 * @param RowText Label for the menu item
	 * @param InOutCurrentState The current state of the toggle
	 * @param Params Additional parameters used to construct the menu button
	 *
	 * @return Whether the user toggled the checkbox this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static bool AddMenuToggleButton(const FString& RowText, UPARAM(Ref) bool& InOutCurrentState, const FSlateIMMenuButtonParams& Params)
	{
		return SlateIM::AddMenuToggleButton(RowText, InOutCurrentState, Params);
	}

	/**
	 * Display a menu item button with a checkmark
	 *
	 * @param RowText Label for the menu item
	 * @param InOutCurrentState The current state of the checkmark
	 * @param Params Additional parameters used to construct the menu button
	 *
	 * @return Whether the user clicked the menu item this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static bool AddMenuCheckButton(const FString& RowText, UPARAM(Ref) bool& InOutCurrentState, const FSlateIMMenuButtonParams& Params)
	{
		return SlateIM::AddMenuCheckButton(RowText, InOutCurrentState, Params);
	}

	/**
	 * Add a submenu item to the current menu
	 *
	 * @param SubMenuText The label for the menu item
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void BeginSubMenu(const FString& SubMenuText)
	{
		SlateIM::BeginSubMenu(SubMenuText);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Menu")
	static void EndSubMenu()
	{
		SlateIM::EndSubMenu();
	}

	/**
	 * Begin a tab group, can contain TabStacks and TabSplitters
	 *
	 * @param TabGroupId Provide a unique identifier for this tab group
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void BeginTabGroup(FName TabGroupId)
	{
		SlateIM::BeginTabGroup(TabGroupId);
	}

	/**
	 * Ends the current Tab Group
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void EndTabGroup()
	{
		SlateIM::EndTabGroup();
	}

	/**
	 * Begin a tab stack, can only contain Tabs
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void BeginTabStack()
	{
		SlateIM::BeginTabStack();
	}

	/**
	 * Ends the current Tab Stack
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void EndTabStack()
	{
		SlateIM::EndTabStack();
	}

	/**
	 * Begin a tab splitter, displays child TabSplitters and TabStacks side-by-side
	 *
	 * @param Orientation The direction to layout child TabSplitters and TabStacks
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void BeginTabSplitter(TEnumAsByte<EOrientation> Orientation)
	{
		SlateIM::BeginTabSplitter(Orientation);
	}

	/**
	 * Ends the current Tab Splitter
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void EndTabSplitter()
	{
		SlateIM::EndTabSplitter();
	}

	/**
	 * Assigns the Size Coefficient for the next child of a TabSplitter
	 *
	 * @param SizeCoefficient The weight to assign to the next child of the parent TabSplitter
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void TabSplitterSizeCoefficient(float SizeCoefficient)
	{
		SlateIM::TabSplitterSizeCoefficient(SizeCoefficient);
	}

	/**
	 * Begins a Tab to contain any other content
	 *
	 * @param TabId Provide a tab id that is unique within the Tab Group
	 * @param Params Additional parameters used to create this tab
	 *
	 * @return Returns true if the tab is active, false otherwise. Logic to draw the contents of the tab can be skipped when this is false.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static bool BeginTab(const FName& TabId, const FSlateIMTabParams& Params)
	{
		return SlateIM::BeginTab(TabId, Params);
	}

	/**
	 * Ends the current Tab. This must be called even when BeginTab returns false.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void EndTab()
	{
		SlateIM::EndTab();
	}

	/**
	 * Causes a one-time activation of a tab in its parent tab well. Can be used at any point during layout.
	 * 
	 * To use this effectively, all tab ids should be unique. Only the first tab registered with the given id will be activated.
	 * Can be called multiple times to activate multiple tabs in the same update.
	 * @see ActivateInParent(ETabActivationCause).
	 * 
	 * @param TabId The tab id to activate.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Tabs")
	static void ActivateTab(const FName& TabId)
	{
		SlateIM::ActivateTab(TabId);
	}

	/**
	 * SlateIM updates are disabled in specific scenarios (like when a SlateIM::ModalDialog is open), use this function to react accordingly
	 *
	 * @return Whether SlateIM can update currently
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Utils")
	static bool CanUpdateSlateIM()
	{
		return SlateIM::CanUpdateSlateIM();
	}

	/**
	 * Disables all widgets until EndDisabledState is called.
	 *
	 * @note It is not possible to enable a widget inside a disabled parent widget by calling EndDisabledState before a child is created inside a disabled Widget.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Utils")
	static void BeginDisabledState()
	{
		SlateIM::BeginDisabledState();
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Utils")
	static void EndDisabledState()
	{
		SlateIM::EndDisabledState();
	}

	/**
	 * Sets the tooltip to be used for the next widget created. Resets after tooltip is used
	 *
	 * @param NextToolTip The tooltip to display
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Utils")
	static void SetToolTip(const FString& NextToolTip)
	{
		SlateIM::SetToolTip(NextToolTip);
	}

	/**
	 * Opens a modal dialog of the specified type. This function will not return until the user closes the modal dialog
	 *
	 * @param MessageType The type of options the user can respond to the dialog with
	 * @param DialogText The message to display to the user
	 * @param Params Additional parameters used to create this dialog
	 *
	 * @return The option the user selected
	 *
	 * @note SlateIM updates are disable while a SlateIM modal is open
	 * @see CanUpdateSlateIM()
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Containers")
	static EAppReturnType::Type ModalDialog(EAppMsgType::Type MessageType, const FString& DialogText, const FSlateIMModalDialogParams& Params)
	{
		return SlateIM::ModalDialog(MessageType, DialogText, Params);
	}
	
	/**
	 * Query whether the last widget is hovered or not
	 * 
	 * @return true if the cursor is hovering the last rendered widget, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Queries")
	static bool IsHovered()
	{
		return SlateIM::IsHovered();
	}

	/**
	 * Query whether the previous widget is focused
	 * 
	 * @param Depth How far to check for focus. Includes child widgets by default.
	 * @return true if the previous widget has focus (for the specified depth), false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Queries", meta=(Depth="/Script/SlateIm.ScriptESlateIMFocusDepth::IncludingDescendants"))
	static bool IsFocused(ESlateIMFocusDepth Depth)
	{
		return SlateIM::IsFocused(Depth);
	}

	/**
	 * Begin a graph widget, call graphing functions between this and EndGraph to include multiple graphs in a single chart
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Graph")
	static void BeginGraph()
	{
		SlateIM::BeginGraph();
	}

	/**
	 * Call when finished calling graphing functions for the widget
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Graph")
	static void EndGraph()
	{
		SlateIM::EndGraph();
	}

	/**
	 * Add a line graph of 2D vectors to the graph widget
	 *
	 * @param Points The X,Y points to plot on the graph
	 * @param Params Additional parameters used to draw the line
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Graph")
	static void GraphLinePoints(const TArray<FVector2D>& Points, const FSlateIMGraphLinePointsParams& Params)
	{
		SlateIM::GraphLine(const_cast<TArray<FVector2D>&>(Points), Params);
	}

	/**
	 * Add a line graph of values to the graph widget.
	 * The value index is used as the X-value when plotting, the graph will scale horizontally to fit all values in the array.
	 *
	 * @param Values The Y values to plot on the graph
	 * @param Params Additional parameters used to draw the line
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Graph")
	static void GraphLineValues(const TArray<double>& Values, const FSlateIMGraphLineValuesParams& Params)
	{
		SlateIM::GraphLine(const_cast<TArray<double>&>(Values), Params);
	}

	/**
	 * Query whether a key was pressed this frame.
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 *
	 * @param InKey The key to query the state of
	 * @return true if the key was just pressed this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Input")
	static bool IsKeyPressed(const FKey& InKey)
	{
		return SlateIM::IsKeyPressed(InKey);
	}

	/**
	 * Query whether a key is being held this frame.
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 *
	 * @param InKey The key to query the state of
	 * @return true if the key was pressed or held this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Input")
	static bool IsKeyHeld(const FKey& InKey)
	{
		return SlateIM::IsKeyHeld(InKey);
	}

	/**
	 * Query whether a key was released this frame.
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 *
	 * @param InKey The key to query the state of
	 * @return true if the key was just released this frame
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Input")
	static bool IsKeyReleased(const FKey& InKey)
	{
		return SlateIM::IsKeyReleased(InKey);
	}

	/**
	 * Retrieve the analog value for a key
	 *
	 * Only considers input events that make it to the SlateIM root. Input events handled by child widgets will not be considered.
	 *
	 * @param InKey The key to query the state of
	 * @return the last analog value the SlateIM root received for the specified key
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Input")
	static float GetKeyAnalogValue(const FKey& InKey)
	{
		return SlateIM::GetKeyAnalogValue(InKey);
	}

	/**
	 * Begin queueing commands to draw to a canvas render target.
	 * 
	 * @param Width The width of the canvas in pixels.
	 * @param Height The height of the canvas in pixels.
	 * @param Params Additional parameters to create this canvas.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void BeginCanvas(int32 Width, int32 Height, const FSlateIMEngineCanvasParams& CanvasParams)
	{
		SlateIM::Canvas::BeginCanvas(Width, Height, CanvasParams);
	}

	/**
	 * Ends the currently drawn canvas and executes all queued commands.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void EndCanvas()
	{
		SlateIM::Canvas::EndCanvas();
	}

	/**
	 * Invalidate the current canvas, forcing a redraw if its UpdateType is set to Invalidation.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void Invalidate()
	{
		SlateIM::Canvas::Invalidate();
	}

	/** Sets the position of the lower-right corner of the clipping region of the canvas */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void SetClip(const FVector2f& ClipPosition)
	{
		SlateIM::Canvas::SetClip(ClipPosition);
	}

	/** Sets the draw color of the canvas. */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void SetLinearDrawColor(const FLinearColor& Color)
	{
		SlateIM::Canvas::SetDrawColor(Color);
	}

	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void SetDrawColor(const FColor& Color)
	{
		SlateIM::Canvas::SetDrawColor(Color);
	}

	/**
	 * Queue drawing a canvas icon (texture) to the canvas.
	 *
	 * @param Icon The icon to draw. The texture draw area is in pixels.
	 * @param CanvasPosition The position of the icon on the canvas, in pixels.
	 * @param Scale The scale of the icon.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawIcon(const FCanvasIcon& Icon, const FVector2f& CanvasPosition, const FVector2f& Scale)
	{
		SlateIM::Canvas::DrawIcon(Icon, CanvasPosition, Scale);
	}

	/**
	 * Queue drawing a line to the canvas.
	 *
	 * @param CanvasPositionA One end of the line on the canvas, in pixels.
	 * @param CanvasPositionB The other end of the line on the canvas, in pixels.
	 * @param Thickness The thickness of the line in pixels.
	 * @param RenderColor The color of the line.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawLine(const FVector2D& CanvasPositionA, const FVector2D& CanvasPositionB, float Thickness, FLinearColor RenderColor)
	{
		SlateIM::Canvas::DrawLine(CanvasPositionA, CanvasPositionB, Thickness, RenderColor);
	}

	/**
	 * Queue drawing a texture to the canvas.
	 *
	 * @param RenderTexture The texture to draw.
	 * @param CanvasPosition The position of the texture on the canvas, in pixels.
	 * @param CanvasSize The size of the texture on the canvas, in pixels.
	 * @param RenderParams Extra optional parameters for tile drawing.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawTexture(UTexture* RenderTexture, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FSlateIMEngineTileRenderParams& RenderParams)
	{
		SlateIM::Canvas::DrawTexture(RenderTexture, CanvasPosition, CanvasSize, RenderParams);
	}

	/**
	 * Queue drawing a material to the canvas.
	 *
	 * @param RenderMaterial The material to draw.
	 * @param CanvasPosition The position of the material on the canvas, in pixels.
	 * @param CanvasSize The size of the material on the canvas, in pixels.
	 * @param RenderParams Extra optional parameters for tile drawing. RenderColor and BlendMode are not used.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawMaterial(UMaterialInterface* RenderMaterial, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FSlateIMEngineTileRenderParams& RenderParams)
	{
		SlateIM::Canvas::DrawMaterial(RenderMaterial, CanvasPosition, CanvasSize, RenderParams);
	}

	/**
	 * Queue drawing text to the canvas.
	 *
	 * @param RenderFont The font used to render the text.
	 * @param RenderText The string to render.
	 * @param CanvasPosition The position of the text on the canvas, in pixels.
	 * @param RenderParams Extra optional parameters for rendering the text.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawText(UFont* RenderFont, const FString& RenderText, const FVector2D& CanvasPosition, const FSlateIMEngineTextRenderParams& RenderParams)
	{
		SlateIM::Canvas::DrawText(RenderFont, RenderText, CanvasPosition, RenderParams);
	}

	/**
	 * Queue drawing text to the canvas.
	 *
	 * Alternate method allowing the use of FFontRenderInfo.
	 *
	 * @param RenderFont The font used to render the text.
	 * @param RenderText The string to render.
	 * @param CanvasPosition The position of the text on the canvas, in pixels.
	 * @param Scale The scale of the text on the canvas.
	 * @param RenderInfo Extra optional parameters for rendering the text.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawTextRenderInfo(UFont* RenderFont, const FString& RenderText, const FVector2f& CanvasPosition, const FVector2f& Scale, const FFontRenderInfo& RenderInfo)
	{
		SlateIM::Canvas::DrawText(RenderFont, RenderText, CanvasPosition, Scale, RenderInfo);
	}

	/**
	 * Queue drawing wrapped text to canvas.
	 *
	 * Uses FFontRenderInfo for text effets.
	 *
	 * Text size / wrapping is automatically computed based on the position of the text and the @see SetCanvasClip.
	 * The default clipping position is the entire canvas. It does not reset between queued calls.
	 *
	 * @param RenderFont The font used to render the text.
	 * @param RenderText The text to render.
	 * @param CanvasPosition The position of the text on the canvas, in pixels.
	 * @param bCenterTextX If true, the text is centered on the given draw position on the X axis.
	 * @param bCenterTextY If true, the text is centered on the given draw position on the Y axis.
	 * @param Scale The scale of the text on the canvas.
	 * @param RenderInfo Extra optional parameters for rendering the text.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawWrappedText(const UFont* RenderFont, const FString& RenderText, const FVector2f& CanvasPosition, bool bCenterTextX, bool bCenterTextY, const FVector2f& Scale, const FFontRenderInfo& RenderInfo)
	{
		SlateIM::Canvas::DrawWrappedText(RenderFont, RenderText, CanvasPosition, bCenterTextX, bCenterTextY, Scale, RenderInfo);
	}

	/**
	 * Queue drawing a border to the canvas.
	 *
	 * @param BorderTexture The texture for the center of the border.
	 * @param BackgroundTexture The texture for the background of the border.
	 * @param LeftBorderTexture The texture for the left of the border.
	 * @param RightBorderTexture The texture for the right of the border.
	 * @param TopBorderTexture The texture for the top of the border.
	 * @param BottomBorderTexture The texture for the bottom of the border.
	 * @param CanvasPosition The position of the border on the canvas, in pixels.
	 * @param CanvasSize The size of the border on the canvas, in pixels.
	 * @param RenderParams Extra optional params for rendering the border.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawBorder(UTexture* BorderTexture, UTexture* BackgroundTexture, UTexture* LeftBorderTexture, UTexture* RightBorderTexture, UTexture* TopBorderTexture, UTexture* BottomBorderTexture, const FVector2D& CanvasPosition, const FVector2D& CanvasSize, const FSlateIMEngineBorderRenderParams& RenderParams)
	{
		SlateIM::Canvas::DrawBorder(BorderTexture, BackgroundTexture, LeftBorderTexture, RightBorderTexture, TopBorderTexture, BottomBorderTexture, CanvasPosition, CanvasSize, RenderParams);
	}


	/**
	 * Queue drawing an unfilled box to the canvas.
	 *
	 * @param CanvasPosition The position of the box on the canvas, in pixels.
	 * @param CanvasSize The size of the box on the canvas, in pixels.
	 * @param Thickness The thicknses of the lines to draw.
	 * @param RenderColor The color of the lines to draw.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawBox(const FVector2D& CanvasPosition, const FVector2D& CanvasSize, float Thickness, FLinearColor RenderColor)
	{
		SlateIM::Canvas::DrawBox(CanvasPosition, CanvasSize, Thickness, RenderColor);
	}

	/**
	 * Queue drawing a filled polygon to the canvas.
	 *
	 * @param RenderTexture The texture to draw with.
	 * @param CanvasPosition The position of the center of the polygon on the canvas, in pixels.
	 * @param Radius The radius of the polygon, in pixels.
	 * @param NumberOfSides The number of sides of the polygon.
	 * @param RenderColor The color of the polygon.
	 */
	UFUNCTION(BlueprintCallable, Category = "SlateIM|Canvas")
	static void DrawPolygon(UTexture* RenderTexture, const FVector2D& CanvasPosition, const FVector2D& Radius, int32 NumberOfSides, FLinearColor RenderColor)
	{
		SlateIM::Canvas::DrawPolygon(RenderTexture, CanvasPosition, Radius, NumberOfSides, RenderColor);
	}
};
