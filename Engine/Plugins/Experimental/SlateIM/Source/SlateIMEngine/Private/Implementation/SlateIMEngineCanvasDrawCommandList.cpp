// Copyright Epic Games, Inc. All Rights Reserved.

#include "Implementation/SlateIMEngineCanvasDrawCommandList.h"

USlateIMEngineCanvasDrawCommandList::USlateIMEngineCanvasDrawCommandList()
	: UpdateType(ESlateIMEngineCanvasUpdateType::EveryFrame)
	, bInvalidated(true)
{
}

ESlateIMEngineCanvasUpdateType USlateIMEngineCanvasDrawCommandList::GetUpdateType() const
{
	return UpdateType;
}

void USlateIMEngineCanvasDrawCommandList::SetUpdateType(ESlateIMEngineCanvasUpdateType InUpdateType)
{
	UpdateType = InUpdateType;
}

void USlateIMEngineCanvasDrawCommandList::Invalidate()
{
	bInvalidated = true;
}

bool USlateIMEngineCanvasDrawCommandList::NeedsUpdate() const
{
	switch (UpdateType)
	{
		case ESlateIMEngineCanvasUpdateType::Invalidation:
			return bInvalidated;

		case ESlateIMEngineCanvasUpdateType::EveryFrame:
			return true;

		default:
			ensureAlwaysMsgf(false, TEXT("Invalid update time [%d]."), static_cast<uint8>(UpdateType));
			return false;
	}
}

void USlateIMEngineCanvasDrawCommandList::EnqueueCommand(FCanvasDrawCommand&& InDrawCommand)
{
	DrawCommands.Add(Forward<FCanvasDrawCommand>(InDrawCommand));
}

void USlateIMEngineCanvasDrawCommandList::ProcessCommands(UCanvas* InCanvas, int32 InWidth, int32 InHeight)
{
	for (const FCanvasDrawCommand& DrawCommand : DrawCommands)
	{
		DrawCommand.ExecuteIfBound(InCanvas);
	}

	// Make sure all delegates are deleted
	DrawCommands.Reset();

	// Reset invalidation as we've now been drawn.
	bInvalidated = false;
}
