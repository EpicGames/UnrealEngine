// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Containers/Array.h"
#include "Delegates/DelegateCombinations.h"
#include "SlateIMEngineParameters.h"

#include "SlateIMEngineCanvasDrawCommandList.generated.h"

class UCanvas;

UCLASS()
class USlateIMEngineCanvasDrawCommandList : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FCanvasDrawCommand, UCanvas*)

	SLATEIMENGINE_API USlateIMEngineCanvasDrawCommandList();

	SLATEIMENGINE_API ESlateIMEngineCanvasUpdateType GetUpdateType() const;

	SLATEIMENGINE_API void SetUpdateType(ESlateIMEngineCanvasUpdateType InUpdateType);

	SLATEIMENGINE_API void Invalidate();

	SLATEIMENGINE_API bool NeedsUpdate() const;

	SLATEIMENGINE_API void EnqueueCommand(FCanvasDrawCommand&& InDrawCommand);

	UFUNCTION()
	SLATEIMENGINE_API void ProcessCommands(UCanvas* InCanvas, int32 InWidth, int32 InHeight);

protected:
	ESlateIMEngineCanvasUpdateType UpdateType;
	bool bInvalidated = false;
	TArray<FCanvasDrawCommand> DrawCommands;
};
