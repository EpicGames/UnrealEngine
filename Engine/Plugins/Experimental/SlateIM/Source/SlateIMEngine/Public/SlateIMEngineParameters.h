// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Math/MathFwd.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "UObject/ObjectPtr.h"

#include "SlateIMEngineParameters.generated.h"

UENUM(BlueprintType)
enum class ESlateIMEngineCanvasUpdateType : uint8
{
	/** Redraw the canvas every frame. */
	EveryFrame,

	/**
	 * Only redraw when specifically invalidated.
	 * Warning: Textures and materials may not be rendered correctly if they are not loaded or shaders are not cached.
	 */
	Invalidation

	/** TODO: Automatically redraw when the hash of the draw queue changes. */
	// HashChange
};

USTRUCT(BlueprintType)
struct FSlateIMEngineCanvasParams
{
	GENERATED_BODY()

	/** Tint applied to the resulting image. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FSlateColor ColorAndOpacity = FSlateColor(EStyleColor::White);

	/** The size of the image drawn in the UI. This may be different from the size of the canvas. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D DesiredSize = FVector2D::ZeroVector;

	/** The world related to the canvas being drawn, if any. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	TObjectPtr<UObject> WorldContext = nullptr;

	/** When the texture will be redrawn. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	ESlateIMEngineCanvasUpdateType UpdateType = ESlateIMEngineCanvasUpdateType::EveryFrame;
};

/**
 * Extra optional parameters for rendering tiles (textures and materials.)
 */
USTRUCT(BlueprintType)
struct FSlateIMEngineTileRenderParams
{
	GENERATED_BODY()

	/** The top left of the area on the tile to draw, in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D UVPosition = FVector2D::ZeroVector;

	/** The size of the area on the tile to draw, in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D UVSize = FVector2D::UnitVector;

	/** A tint to apply to tile. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FLinearColor RenderColor = FLinearColor::White;

	/** The blend mode with which to draw the tile. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	TEnumAsByte<EBlendMode> BlendMode = BLEND_Translucent;

	/** The rotation of the tile, in degrees. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	float Rotation = 0.f;

	/** The point on the tile around which it is rotated, in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D PivotPoint = FVector2D(0.5, 0.5);
};

/**
 * Extra optional parameters for rendering text.
 */
USTRUCT(BlueprintType)
struct FSlateIMEngineTextRenderParams
{
	GENERATED_BODY()

	/** The scale of the text on the canvas. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D Scale = FVector2D(1.0f, 1.0f);

	/** The color of the text. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FLinearColor RenderColor = FLinearColor::White;

	/** Adjust the distance between letters. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	float Kerning = 0.0f;

	/** The color of the text shadow. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FLinearColor ShadowColor = FLinearColor::Black;

	/** The offset of the shadow from the text it is shadowing. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D ShadowOffset = FVector2D::UnitVector;

	/** If true, the text is centered on the given draw position on the X axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	bool bCentreX = false;

	/** If true, the text is centered on the given draw position on the Y axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	bool bCentreY = false;

	/** If true, outline the text. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	bool bOutlined = false;

	/** Color of the text outline. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FLinearColor OutlineColor = FLinearColor::Black;
};

/**
 * Extra optional parameters for rendering borders.
 *
 * Tile refers to the centre texture.
 */
USTRUCT(BlueprintType)
struct FSlateIMEngineBorderRenderParams
{
	GENERATED_BODY()

	/** The top left of the area on the tile to draw, in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D UVPosition = FVector2D::ZeroVector;

	/** The size of the area on the tile to draw, in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D UVSize = FVector2D::UnitVector;

	/** A tint to apply to tile. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FLinearColor RenderColor = FLinearColor::White;

	/** The blend mode with which to draw the tile. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	TEnumAsByte<EBlendMode> BlendMode = BLEND_Translucent;

	/** The rotation of the tile, in degrees. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	float Rotation = 0.f;

	/** The point on the tile around which it is rotated, in texels. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D PivotPoint = FVector2D(0.5, 0.5);

	/** The scale of the border. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D BorderScale = FVector2D(0.1, 0.1);

	/** The scale of the background. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D BackgroundScale = FVector2D(0.1, 0.1);

	/** Frame corner size, in percent (should be < 0.5f). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SlateIMEngine")
	FVector2D CornerSize = FVector2D::ZeroVector;
};
