// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateIMEngine : ModuleRules
{
	public SlateIMEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"SlateCore",
			}
		);
	}
}
