// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMInGameNomadTab.h"

#include "SlateIM.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateIMInGameNomadTab)

void ASlateIMInGameNomadTab::Init(const FName& InTabName, const FStringView& InTabTitle, const FSlateIcon& InTabIcon)
{
	TabName = InTabName;
	TabTitle = FText::FromStringView(InTabTitle);
	TabIcon = InTabIcon;
}

void ASlateIMInGameNomadTab::OnWidgetStarted()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabName, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.TabRole(ETabRole::NomadTab)
			.Label(TabTitle)
			.OnTabActivated_Lambda([this](TSharedRef<SDockTab>, ETabActivationCause) 
				{ OnTabActivated(); })
			.OnTabClosed_Lambda([this](TSharedRef<SDockTab>) 
				{ OnTabDeactivated(); });
	}))
	.SetIcon(TabIcon);
}

void ASlateIMInGameNomadTab::OnWidgetStopped()
{
	if (TSharedPtr<SDockTab> RootTabPin = RootTab.Pin())
	{
		RootTabPin->SetOnTabActivated(SDockTab::FOnTabActivatedCallback());
		RootTabPin->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
		RootTabPin->RequestCloseTab();
		RootTab.Reset();
		TabContent.Reset();
	}

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabName);
}

void ASlateIMInGameNomadTab::DrawWidget(const float DeltaTime)
{
	if (!SlateIM::CanUpdateSlateIM())
	{
		return;
	}

	TSharedPtr<SDockTab> RootTabPin = RootTab.Pin();
	if (!RootTabPin.IsValid())
	{
		// This is the first time we call DrawWidget() since being enabled - invoke the tab
		RootTab = RootTabPin = FGlobalTabmanager::Get()->TryInvokeTab(TabName);
		check(RootTabPin.IsValid())
	}

	TSharedPtr<SWidget> NewExposedWidget;
	if (SlateIM::BeginExposedRoot(TabName, NewExposedWidget))
	{
		DrawContent(DeltaTime);
	}
	SlateIM::EndRoot();

	if (NewExposedWidget.IsValid())
	{
		if (NewExposedWidget != TabContent)
		{
			RootTabPin->SetContent(NewExposedWidget.ToSharedRef());
			TabContent = NewExposedWidget;
		}
	}
	else
	{
		RootTabPin->RequestCloseTab();
		OnTabDeactivated();
	}
}

void ASlateIMInGameNomadTab::OnTabActivated()
{
}

void ASlateIMInGameNomadTab::OnTabDeactivated()
{
	TabContent.Reset();
	if (!bDestroyRequested)
	{
		bDestroyRequested = true;
		Server_Destroy();
	}
}
