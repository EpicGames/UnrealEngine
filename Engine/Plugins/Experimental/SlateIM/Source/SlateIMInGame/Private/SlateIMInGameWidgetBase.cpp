// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMInGameWidgetBase.h"

#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerController.h"
#include "SlateIM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateIMInGameWidgetBase)

ASlateIMInGameWidgetBase::ASlateIMInGameWidgetBase()
{
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bStartWithTickEnabled = false;

	bReplicates = true;
	bAlwaysRelevant = true;

	SetNetUpdateFrequency(3.f);
	SetMinNetUpdateFrequency(3.f);
}

void ASlateIMInGameWidgetBase::BeginPlay()
{
	Super::BeginPlay();

	if (IsLocallyControlled())
	{
		StartWidget();
	}
}

void ASlateIMInGameWidgetBase::EndPlay(EEndPlayReason::Type Reason)
{
	if (IsLocallyControlled())
	{
		StopWidget();
	}

	Super::EndPlay(Reason);
}

void ASlateIMInGameWidgetBase::OnRep_Owner()
{
	Super::OnRep_Owner();

	if (HasActorBegunPlay())
	{
		if (IsLocallyControlled())
		{
			StartWidget();
		}
		else
		{
			StopWidget();
		}
	}
}

void ASlateIMInGameWidgetBase::PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker)
{
	Super::PreReplication(ChangedPropertyTracker);

#if WITH_SERVER_CODE
	if (HasAuthority())
	{
		GenerateServerSnapshot();
	}
#endif // WITH_SERVER_CODE
}

bool ASlateIMInGameWidgetBase::IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const
{
	return GetOwner() != nullptr && GetOwner() == RealViewer;
}

APlayerController* ASlateIMInGameWidgetBase::GetPlayerController() const
{
	return Cast<APlayerController>(GetOwner());
}

bool ASlateIMInGameWidgetBase::IsLocallyControlled() const
{
	if (APlayerController* OwnerPlayerController = GetPlayerController())
	{
		return OwnerPlayerController->IsLocalController();
	}

	return false;
}

ASlateIMInGameWidgetBase* ASlateIMInGameWidgetBase::GetInGameWidget(const APlayerController* Owner, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass)
{
	if (ensure(Owner && InGameWidgetClass))
	{
		for (TActorIterator<ASlateIMInGameWidgetBase> It(Owner->GetWorld(), InGameWidgetClass); It; ++It)
		{
			if (It->GetOwner() == Owner)
			{
				return *It;
			}
		}
	}
	return nullptr;
}

void ASlateIMInGameWidgetBase::EnableInGameWidget(APlayerController* Owner, const bool bEnable, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass)
{
	if (bEnable)
	{
		GetOrOpenInGameWidget(Owner, InGameWidgetClass);
	}
	else 
	{
		DestroyInGameWidget(Owner, InGameWidgetClass);
	}
}

void ASlateIMInGameWidgetBase::StartWidget()
{
	if (!WidgetTickHandle.IsValid())
	{
		WidgetTickHandle = FSlateApplication::Get().OnPreTick().AddUObject(this, &ASlateIMInGameWidgetBase::TickWidget);

		OnWidgetStarted();
	}
}

void ASlateIMInGameWidgetBase::StopWidget()
{
	if (WidgetTickHandle.IsValid())
	{
		FSlateApplication::Get().OnPreTick().Remove(WidgetTickHandle);
		WidgetTickHandle.Reset();

		OnWidgetStopped();
	}
}

void ASlateIMInGameWidgetBase::TickWidget(const float DeltaTime)
{
	if (!SlateIM::CanUpdateSlateIM())
	{
		return;
	}

	DrawWidget(DeltaTime);
}

ASlateIMInGameWidgetBase* ASlateIMInGameWidgetBase::GetOrOpenInGameWidget(APlayerController* Owner, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass)
{
	if (ensure(Owner && InGameWidgetClass))
	{
		if (ASlateIMInGameWidgetBase* ExistingWidget = GetInGameWidget(Owner, InGameWidgetClass))
		{
			return ExistingWidget;
		}

		if (GetDefault<ASlateIMInGameWidgetBase>(InGameWidgetClass)->bReplicates && Owner->GetWorld()->GetNetMode() == ENetMode::NM_Client)
		{
			UE_LOGF(LogActor, Error, "In game widget %ls was requested but it's replicated and needs server permission.", *InGameWidgetClass->GetName());
		}
		else
		{
			FActorSpawnParameters Params;
			Params.Owner = Owner;

			if (ASlateIMInGameWidgetBase* NewWidget = Owner->GetWorld()->SpawnActor<ASlateIMInGameWidgetBase>(InGameWidgetClass, Params))
			{
				NewWidget->AttachToActor(Owner, FAttachmentTransformRules::KeepRelativeTransform);
				return NewWidget;
			}
		}
	}

	return nullptr;
}

void ASlateIMInGameWidgetBase::DestroyInGameWidget(const APlayerController* Owner, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass)
{
	if (ensure(Owner && InGameWidgetClass))
	{
		if (ASlateIMInGameWidgetBase* OpenedWidget = GetInGameWidget(Owner, InGameWidgetClass))
		{
			if (GetDefault<ASlateIMInGameWidgetBase>(InGameWidgetClass)->bReplicates && Owner->GetWorld()->GetNetMode() == ENetMode::NM_Client)
			{
				UE_LOGF(LogActor, Error, "Destruction of in game widget %ls was requested but it is replicated and needs server permission.", *InGameWidgetClass->GetName());
			}
			else
			{
				OpenedWidget->Destroy();
			}
		}
	}
}

void ASlateIMInGameWidgetBase::Server_Destroy_Implementation()
{
	Destroy();
}
