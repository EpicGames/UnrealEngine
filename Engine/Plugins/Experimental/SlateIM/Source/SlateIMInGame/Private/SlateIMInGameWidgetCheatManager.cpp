// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMInGameWidgetCheatManager.h"

#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "SlateIMInGameWidgetBase.h"
#include "SlateIMInGameWidgetModularFeature.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateIMInGameWidgetCheatManager)

USlateIMInGameWidgetCheatManager::USlateIMInGameWidgetCheatManager()
{
#if UE_WITH_CHEAT_MANAGER
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UCheatManager::RegisterForOnCheatManagerCreated(FOnCheatManagerCreated::FDelegate::CreateLambda(
			[](UCheatManager* CheatManager)
			{
				CheatManager->AddCheatManagerExtension(NewObject<ThisClass>(CheatManager));
			}));
	}
#endif
}

void USlateIMInGameWidgetCheatManager::ToggleSlateIMInGameWidget(const FString& Path) const
{
#if UE_WITH_CHEAT_MANAGER
	APlayerController* PlayerController = GetPlayerController();
	if (!PlayerController)
	{
		return;
	}

	const TSoftClassPtr<ASlateIMInGameWidgetBase> InGameWidgetClass = FSlateIMInGameWidgetModularFeature::FindWidgetClass(SlateIMInGameWidget::ModularFeatureName, Path);
	if (InGameWidgetClass.Get() == nullptr)
	{
		UE_LOGF(LogConsoleResponse, Warning, "No in game widget with path '%ls'", *Path);
		return;
	}

	const bool bIsMenuEnabled = ASlateIMInGameWidgetBase::GetInGameWidget(PlayerController, InGameWidgetClass.Get()) != nullptr;
	// If we're not Authority, we should be sending this cheat command to the server to execute
	if (PlayerController->GetLocalRole() != ENetRole::ROLE_Authority)
	{
		PlayerController->ServerExec(FString::Printf(TEXT("EnableInGameWidgetFromClass %s %d"), *InGameWidgetClass.ToString(), bIsMenuEnabled ? 0 : 1));
		return;
	}

	EnableInGameWidgetFromClass(InGameWidgetClass.ToString(), !bIsMenuEnabled);
#endif
}

void USlateIMInGameWidgetCheatManager::EnableInGameWidgetFromClass(const FString& ClassPath, const bool bEnable) const
{
#if UE_WITH_CHEAT_MANAGER
	APlayerController* PlayerController = GetPlayerController();
	if (!PlayerController)
	{
		return;
	}

	const TSoftClassPtr<ASlateIMInGameWidgetBase> WidgetClass(ClassPath);
	UClass* MenuClass = WidgetClass.LoadSynchronous();
	if (!MenuClass)
	{
		return;
	}

	ASlateIMInGameWidgetBase::EnableInGameWidget(PlayerController, bEnable, MenuClass);
#endif
}
