// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/CheatManager.h"

#include "SlateIMInGameWidgetCheatManager.generated.h"

UCLASS()
class USlateIMInGameWidgetCheatManager : public UCheatManagerExtension
{
	GENERATED_BODY()

public:
	USlateIMInGameWidgetCheatManager();

	UFUNCTION(Exec)
	void ToggleSlateIMInGameWidget(const FString& Path) const;

private:
	UFUNCTION(Exec)
	void EnableInGameWidgetFromClass(const FString& ClassPath, const bool bEnable) const;
};