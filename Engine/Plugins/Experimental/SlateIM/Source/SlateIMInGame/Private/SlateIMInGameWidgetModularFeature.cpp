// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMInGameWidgetModularFeature.h"

#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/CheatManager.h"
#include "SlateIMInGameWidgetBase.h"
#include "SlateIMInGameWidgetCheatManager.h"

const FLazyName SlateIMInGameWidget::ModularFeatureName = TEXT("SlateIMInGameWidgets");

FSlateIMInGameWidgetModularFeature::FSlateIMInGameWidgetModularFeature(const FString& InPath, const TSubclassOf<ASlateIMInGameWidgetBase>& InWidgetClass)
	: Path(InPath)
	, WidgetClass(InWidgetClass.Get())
	, WidgetCommand(*FString::Printf(TEXT("SlateIM.ToggleInGameWidget.%s"), *InPath), TEXT(""), FConsoleCommandWithWorldDelegate::CreateRaw(this, &FSlateIMInGameWidgetModularFeature::ToggleWidget))
{
}

TSoftClassPtr<ASlateIMInGameWidgetBase> FSlateIMInGameWidgetModularFeature::FindWidgetClass(const FName Type, const FString& InPath)
{
	TArray<FSlateIMInGameWidgetModularFeature*> DebugWidgetModularFeatures = IModularFeatures::Get().GetModularFeatureImplementations<FSlateIMInGameWidgetModularFeature>(Type);
	for (FSlateIMInGameWidgetModularFeature* DebugWidgetModularFeature : DebugWidgetModularFeatures)
	{
		if (!DebugWidgetModularFeature)
		{
			continue;
		}

		if (!InPath.Equals(DebugWidgetModularFeature->Path, ESearchCase::IgnoreCase))
		{
			continue;
		}

		return DebugWidgetModularFeature->WidgetClass;
	}

	return {};
}

void FSlateIMInGameWidgetModularFeature::ToggleWidget(UWorld* World)
{
	for (FConstPlayerControllerIterator Iterator=World->GetPlayerControllerIterator(); Iterator; ++Iterator)
	{
		if (APlayerController* PlayerController = Iterator->Get())
		{
			if (PlayerController->IsLocalController())
			{
				if (USlateIMInGameWidgetCheatManager* WidgetCheatManager = PlayerController->CheatManager->FindCheatManagerExtension<USlateIMInGameWidgetCheatManager>())
				{
					WidgetCheatManager->ToggleSlateIMInGameWidget(Path);
					return;
				}
			}
		}
	}
}
