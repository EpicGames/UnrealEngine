// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateIMInGameWindow.h"

#include "SlateIM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateIMInGameWindow)

void ASlateIMInGameWindow::Init(const FName& InWindowName, const FStringView& InWindowTitle)
{
	WindowName = InWindowName;
	WindowTitle = InWindowTitle;
}

void ASlateIMInGameWindow::DrawWidget(const float DeltaTime)
{
	const bool bIsDrawingWindow = SlateIM::BeginWindowRoot(
		WindowName,
		{
			.WindowTitle = WindowTitle,
			.WindowSize = WindowSize
		}
	);

	if (bIsDrawingWindow)
	{
		DrawContent(DeltaTime);
	}
	SlateIM::EndRoot();

	if (!bIsDrawingWindow && !bDestroyRequested)
	{
		bDestroyRequested = true;
		Server_Destroy();
	}
}
