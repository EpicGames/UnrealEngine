// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateIMInGameWidgetBase.h"

#include "SlateIMInGameNomadTab.generated.h"

#define UE_API SLATEIMINGAME_API

UCLASS(MinimalAPI)
class ASlateIMInGameNomadTab : public ASlateIMInGameWidgetBase
{
	GENERATED_BODY()

protected:
	UE_API void Init(const FName& InTabName, const FStringView& InTabTitle, const FSlateIcon& InTabIcon);
	virtual void DrawContent(const float DeltaTime) {};
	UE_API virtual void OnWidgetStarted() override;
	UE_API virtual void OnWidgetStopped() override;

private:
	UE_API virtual void DrawWidget(const float DeltaTime) override;

	void OnTabActivated();
	void OnTabDeactivated();

	FName TabName;
	FText TabTitle;
	FSlateIcon TabIcon;
	TWeakPtr<SDockTab> RootTab;
	TWeakPtr<SWidget> TabContent;
	bool bDestroyRequested = false;
};

#undef UE_API
