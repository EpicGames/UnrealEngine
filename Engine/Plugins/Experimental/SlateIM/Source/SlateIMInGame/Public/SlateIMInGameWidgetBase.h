// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "SlateIMWidgetBase.h"

#include "SlateIMInGameWidgetBase.generated.h"

#define UE_API SLATEIMINGAME_API

class APlayerController;

UCLASS(MinimalAPI)
class ASlateIMInGameWidgetBase : public AActor
{
	GENERATED_BODY()

public:
	UE_API ASlateIMInGameWidgetBase();

	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(EEndPlayReason::Type Reason) override;

	UE_API virtual void OnRep_Owner() override;
	UE_API virtual void PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker) override;
	UE_API virtual bool IsNetRelevantFor(const AActor* RealViewer, const AActor* ViewTarget, const FVector& SrcLocation) const override;

	UE_API APlayerController* GetPlayerController() const;
	bool IsLocallyControlled() const;

	static UE_API ASlateIMInGameWidgetBase* GetInGameWidget(const APlayerController* Owner, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass);
	static UE_API void EnableInGameWidget(APlayerController* Owner, const bool bEnable, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass);

protected:
#if WITH_SERVER_CODE
	virtual void GenerateServerSnapshot() {}
#endif

	virtual void DrawWidget(const float DeltaTime) {}

	UE_API void StartWidget();
	UE_API void StopWidget();
	void TickWidget(const float DeltaTime);

	virtual void OnWidgetStarted() {}
	virtual void OnWidgetStopped() {}

	static ASlateIMInGameWidgetBase* GetOrOpenInGameWidget(APlayerController* Owner, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass);
	static void DestroyInGameWidget(const APlayerController* Owner, const TSubclassOf<ASlateIMInGameWidgetBase>& InGameWidgetClass);

	FDelegateHandle WidgetTickHandle;

	UFUNCTION(Reliable, Server)
	UE_API void Server_Destroy();
};

#undef UE_API
