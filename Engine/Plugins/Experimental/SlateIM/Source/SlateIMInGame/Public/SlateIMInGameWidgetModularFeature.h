// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "HAL/IConsoleManager.h"
#include "Templates/SubclassOf.h"
#include "UObject/SoftObjectPtr.h"

#define UE_API SLATEIMINGAME_API

class ASlateIMInGameWidgetBase;

namespace SlateIMInGameWidget
{
	extern UE_API const FLazyName ModularFeatureName;
}

class FSlateIMInGameWidgetModularFeature : public IModularFeature
{
public:
	UE_API FSlateIMInGameWidgetModularFeature(const FString& InPath, const TSubclassOf<ASlateIMInGameWidgetBase>& InWidgetClass);

	static TSoftClassPtr<ASlateIMInGameWidgetBase> FindWidgetClass(const FName Type, const FString& InPath);

protected:
	void ToggleWidget(UWorld* World);

	FString Path;
	TSoftClassPtr<ASlateIMInGameWidgetBase> WidgetClass;
	FAutoConsoleCommandWithWorld WidgetCommand;
};

#undef UE_API