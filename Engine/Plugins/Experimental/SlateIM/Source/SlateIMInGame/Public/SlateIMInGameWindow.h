// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateIMInGameWidgetBase.h"

#include "SlateIMInGameWindow.generated.h"

#define UE_API SLATEIMINGAME_API

UCLASS(MinimalAPI)
class ASlateIMInGameWindow : public ASlateIMInGameWidgetBase
{
	GENERATED_BODY()

protected:
	UE_API void Init(const FName& InWindowName, const FStringView& InWindowTitle);

	virtual void DrawContent(const float DeltaTime) {};

private:
	UE_API virtual void DrawWidget(const float DeltaTime) override;

	FName WindowName;
	FString WindowTitle;
	FVector2f WindowSize = FVector2f(500,500);
	bool bDestroyRequested = false;
};

#undef UE_API
